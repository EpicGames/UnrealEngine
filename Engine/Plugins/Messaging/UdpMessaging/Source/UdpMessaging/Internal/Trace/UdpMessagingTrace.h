// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MessagingTrace.h"
#include "Trace/Trace.h"


enum class EMessageFlags : uint32;
class FName;
struct FGuid;
struct FIPv4Endpoint;
struct FTopLevelAssetPath;
template<typename T, typename... Ts> class TVariant;


#if !defined(UDPMESSAGINGTRACE_ENABLED)
#	define UDPMESSAGINGTRACE_ENABLED MESSAGINGTRACE_ENABLED
#endif


namespace UE::UdpMessaging::Trace
{
	enum class EUdpMessageLifecycleEvent : uint8
	{
		// Send lifecycle
		EnqueueOutbound,
		BeginSerialize,
		EndSerialize,
		ConsumeOutbound,
		Complete,

		// Receive lifecycle
		FirstSegmentEnqueueInbound,
		BeginReassembly,
		EndReassembly,
		PendingDelivery,
		BeginDeserialize,
		Delivered,

		// Exceptional (send or receive)
		Discarded,
	};

	UDPMESSAGING_API const TCHAR* LexToString(EUdpMessageLifecycleEvent InEvent);
}


#if UDPMESSAGINGTRACE_ENABLED


namespace UE::UdpMessaging::Trace
{
	void OutputDiscoveredNode(uint16 ShortId, uint16 DiscoveredByShortId, const FGuid& NodeId, const FIPv4Endpoint& Endpoint, uint8 ProtocolVersion);

	void OutputMessageSummary(uint16 SenderShortId, uint16 RecipientShortId, uint32 MessageId, uint64 Size, uint32 NumSegments, EMessageFlags Flags);
	void OutputMessageTypeInfo(uint16 SenderShortId, uint16 RecipientShortId, uint32 MessageId, const TVariant<FName, FTopLevelAssetPath>& TypeInfo);
	void OutputMessageLifecycle(uint64 Cycle, uint16 SenderShortId, uint16 RecipientShortId, uint32 MessageId, EUdpMessageLifecycleEvent Event);
}


#define TRACE_UDPMESSAGING_DISCOVERED_NODE(ShortId, DiscoveredByShortId, NodeId, Endpoint, ProtocolVersion) \
	UE::UdpMessaging::Trace::OutputDiscoveredNode(ShortId, DiscoveredByShortId, NodeId, Endpoint, ProtocolVersion);

#define TRACE_UDPMESSAGING_MESSAGE_SUMMARY(SenderShortId, RecipientShortId, MessageId, Size, NumSegments, Flags) \
	UE::UdpMessaging::Trace::OutputMessageSummary(SenderShortId, RecipientShortId, MessageId, Size, NumSegments, Flags);

#define TRACE_UDPMESSAGING_MESSAGE_TYPEINFO(SenderShortId, RecipientShortId, MessageId, TypeInfo) \
	UE::UdpMessaging::Trace::OutputMessageTypeInfo(SenderShortId, RecipientShortId, MessageId, TypeInfo);

#define TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(Cycle, SenderShortId, RecipientShortId, MessageId, Event) \
	UE::UdpMessaging::Trace::OutputMessageLifecycle(Cycle, SenderShortId, RecipientShortId, MessageId, Event);


#else // #if UDPMESSAGINGTRACE_ENABLED


#define TRACE_UDPMESSAGING_DISCOVERED_NODE(...)
#define TRACE_UDPMESSAGING_MESSAGE_SUMMARY(...)
#define TRACE_UDPMESSAGING_MESSAGE_TYPEINFO(...)
#define TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(...)


#endif // #if UDPMESSAGINGTRACE_ENABLED .. #else
