// Copyright Epic Games, Inc. All Rights Reserved.

#include "SIpAddressComboBox.h"

#if WITH_EDITOR

#include "Framework/Views/TableViewMetadata.h"
#include "SocketSubsystem.h" 
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SEditableText.h"

namespace UE::UdpMessaging::Private
{
	static TArray<TSharedPtr<FString>> GetLocalNetworkInterfaceCardIPs()
	{
		TArray<TSharedPtr<FString>> LocalNetworkInterfaceCardIPs;
	#if PLATFORM_WINDOWS
		// Add the default route IP Address, only for windows
		const FString DefaultRouteLocalAdapterAddress = TEXT("0.0.0.0:0");
		LocalNetworkInterfaceCardIPs.Add(MakeShared<FString>(DefaultRouteLocalAdapterAddress));
	#endif 
		// Add the local host IP address
		const FString LocalHostIpAddress = TEXT("127.0.0.1:0");
		LocalNetworkInterfaceCardIPs.Add(MakeShared<FString>(LocalHostIpAddress));

		TArray<TSharedPtr<FInternetAddr>> Addresses;
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalAdapterAddresses(Addresses);
		for (TSharedPtr<FInternetAddr> Address : Addresses)
		{
			// Add unique, so in ase the OS call returns with the local host or default route IP, we don't add it twice
			LocalNetworkInterfaceCardIPs.AddUnique(MakeShared<FString>(Address->ToString(false) + ":0"));
		}

		return LocalNetworkInterfaceCardIPs;
	}
}

void SIpAddressComboBox::Construct(const FArguments& InArgs)
{
	OnIPAddressSelectedDelegate = InArgs._OnIPAddressSelected;
	LocalAdapterAddressSource = UE::UdpMessaging::Private::GetLocalNetworkInterfaceCardIPs();
	ChildSlot
	[
		SAssignNew(LocalAdapterAddressComboBox, SComboBox<TSharedPtr<FString>>)
		.OptionsSource(&LocalAdapterAddressSource)
		.OnGenerateWidget(this, &SIpAddressComboBox::GenerateLocalAdapterAddressComboBoxEntry)
		.OnSelectionChanged(this, &SIpAddressComboBox::OnIpAddressSelected)
		.Content()
		[
			SAssignNew(IPAddressEditableTextBlock, SEditableText)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
			.Text(FText::FromString(InArgs._InitialValue))
			.OnTextCommitted(this, &SIpAddressComboBox::OnIPAddressTextCommmited)
		]
	];	
}

void SIpAddressComboBox::OnIpAddressSelected(TSharedPtr<FString> InAddress, ESelectInfo::Type InType)
{
	TSharedPtr<FString> SelectedIPAddress = LocalAdapterAddressComboBox->GetSelectedItem();
	if (SelectedIPAddress.IsValid())
	{
		IPAddressEditableTextBlock->SetText(FText::FromString(*SelectedIPAddress));
		OnIPAddressSelectedDelegate.ExecuteIfBound(*InAddress.Get());
	}
}

void SIpAddressComboBox::OnIPAddressTextCommmited(const FText& Text, ETextCommit::Type)
{
	OnIPAddressSelectedDelegate.ExecuteIfBound(Text.ToString());
}

TSharedRef<SWidget> SIpAddressComboBox::GenerateLocalAdapterAddressComboBoxEntry(TSharedPtr<FString> InAddress)
{
	return
		SNew(STextBlock)
		.Text(FText::FromString(*InAddress))
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")));
}

#endif
