// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Widgets/SCompoundWidget.h"

class SEditableText;
namespace ESelectInfo { enum Type : int; }
namespace ETextCommit { enum Type : int; }
template <typename OptionType> class SComboBox;

class SEditableTextBox;

class SIpAddressComboBox : public SCompoundWidget
{
public:

	DECLARE_DELEGATE_OneParam(FOnChooseIP, FString)
	
	SLATE_BEGIN_ARGS(SIpAddressComboBox)
	{}
	SLATE_ARGUMENT(FString, InitialValue)
	SLATE_EVENT(FOnChooseIP, OnIPAddressSelected)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
	
private:
	
	TArray<TSharedPtr<FString>> LocalAdapterAddressSource;

	TSharedPtr<SComboBox<TSharedPtr<FString>>> LocalAdapterAddressComboBox;
	
	/** Text box shown on top of the local adapter address address combo box */
	TSharedPtr<SEditableText> IPAddressEditableTextBlock;

	/** Delegate executed when a local IP address was selected */
	FOnChooseIP OnIPAddressSelectedDelegate;

	/** Handles changes in the local adapter address combo box */
	void OnIpAddressSelected(TSharedPtr<FString> InAddress, ESelectInfo::Type InType);

	/** Called when the IP Address was commited in the editable text block */
	void OnIPAddressTextCommmited(const FText&, ETextCommit::Type);

	/** Generates an entry in the local adapter address combo box */
	TSharedRef<SWidget> GenerateLocalAdapterAddressComboBoxEntry(TSharedPtr<FString> InAddress);
};

#endif
