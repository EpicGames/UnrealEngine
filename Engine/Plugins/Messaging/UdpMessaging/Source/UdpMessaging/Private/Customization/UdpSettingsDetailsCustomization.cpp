// Copyright Epic Games, Inc. All Rights Reserved.

#include "UdpSettingsDetailsCustomization.h"

#if WITH_EDITOR
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "SIpAddressComboBox.h"
#include "Shared/UdpMessagingSettings.h"

TSharedRef<IDetailCustomization> FUdpSettingsDetailsCustomization::MakeInstance()
{
	return MakeShared<FUdpSettingsDetailsCustomization>();
}

void FUdpSettingsDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	UUdpMessagingSettings* Settings = GetMutableDefault<UUdpMessagingSettings>();
	CustomizeStringAsIpDropDown(Settings->UnicastEndpoint, DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(UUdpMessagingSettings, UnicastEndpoint)), DetailBuilder);
}

void FUdpSettingsDetailsCustomization::CustomizeStringAsIpDropDown(const FString& InitialValue, TSharedPtr<IPropertyHandle> PropertyHandle, IDetailLayoutBuilder& DetailBuilder)
{
	IDetailPropertyRow* Row = DetailBuilder.EditDefaultProperty(PropertyHandle);

	TSharedPtr<SWidget> NameWidget, ValueWidget;
	Row->GetDefaultWidgets(NameWidget, ValueWidget);
	
	Row->CustomWidget()
		.NameContent()
		[
			NameWidget->AsShared()
		]
		.ValueContent()
		[
			SNew(SIpAddressComboBox)
			.InitialValue(InitialValue)
			.OnIPAddressSelected_Lambda([PropertyHandle](const FString& SelectedIp)
			{
				PropertyHandle->SetPerObjectValue(0, SelectedIp);
			})
		];
}
#endif
