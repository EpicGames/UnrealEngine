// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR
#include "IDetailCustomization.h"

class IPropertyHandle;

class FUdpSettingsDetailsCustomization : public IDetailCustomization
{
public:
	
	static TSharedRef<IDetailCustomization> MakeInstance();
	
	//~ Begin IDetailCustomization Interface
	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
	//~ End IDetailCustomization Interface

private:

	void CustomizeStringAsIpDropDown(const FString& InitialValue, TSharedPtr<IPropertyHandle> PropertyHandle, IDetailLayoutBuilder& DetailBuilder);
};
#endif
