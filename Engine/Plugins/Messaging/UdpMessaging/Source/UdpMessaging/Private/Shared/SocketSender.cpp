// Copyright Epic Games, Inc. All Rights Reserved.

#include "SocketSender.h"

#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "Logging/StructuredLog.h"
#include "Sockets.h"
#include "SocketSubsystem.h"
#include "Trace/UdpMessagingTrace.h"
#include "UdpMessagingPrivate.h"


namespace UE::UdpMessaging {


FSocketSender::FSocketSender(FSocket* InSocket, const TCHAR* InDescription, const FOptions& InOptions /* = FOptions() */)
	: Socket(InSocket)
	, SocketSubsystem(ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM))
	, SocketAddr(SocketSubsystem->CreateInternetAddr())
	, bStopping(false)
	, WaitTime(InOptions.WaitTime)
	, Description(InDescription)
	, TokenBucket(InOptions.BytesPerSec, InOptions.MaxBurstBytes)
{
	check(Socket != nullptr);
	check(Socket->GetSocketType() == SOCKTYPE_Datagram);

	Socket->GetAddress(*SocketAddr);

	int32 NewSize = 0;
	Socket->SetSendBufferSize(InOptions.SendBufferSize, NewSize);

	WorkEvent = FPlatformProcess::GetSynchEventFromPool();
	Thread = FRunnableThread::Create(this, InDescription, 128 * 1024, TPri_AboveNormal,
		FPlatformAffinity::GetPoolThreadMask());

	if (InOptions.SegmentOffloadSize != 0)
	{
		TryEnableSegmentationOffload(InOptions.SegmentOffloadSize);
	}
	else
	{
		SegmentOffloadSize = 0;
	}
}


FSocketSender::~FSocketSender()
{
	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
	}

	FPlatformProcess::ReturnSynchEventToPool(WorkEvent);
	WorkEvent = nullptr;
}


bool FSocketSender::Send(
	const TSharedRef<TArray<uint8>>& InData,
	const FIPv4Endpoint& InRecipient,
	bool bAllowSegmentation, /* = false */
	bool bHighPriority /* = false */
)
{
	if (!bStopping)
	{
		if (!bAllowSegmentation && SegmentOffloadSize > 0)
		{
			// If this hits, your call to Send() is going to generate multiple packets on the wire.
			// If this was intentional, pass bAllowSegmentation = true.
			checkf(InData->Num() <= SegmentOffloadSize,
				TEXT("Send data exceeds USO offload size, but bAllowSegmentation was false"));
		}

		TMpscQueue<FPacket>& Queue = bHighPriority ? PrioritySendQueue : SendQueue;
		Queue.Enqueue(InData, InRecipient);
		WorkEvent->Trigger();
		return true;
	}

	return false;
}

uint32 FSocketSender::Run()
{
	while (!bStopping)
	{
		const EUpdateResult Result = Update(WaitTime);
		switch (Result)
		{
			case EUpdateResult::Done:
				WorkEvent->Wait(WaitTime);
				continue;
			case EUpdateResult::Retry:
				continue;
			case EUpdateResult::Fatal:
				bStopping = true;
				return 0;
		}
	}

	return 0;
}


void FSocketSender::Stop()
{
	bStopping = true;
	WorkEvent->Trigger();
}


FSocketSender::EUpdateResult FSocketSender::Update(const FTimespan& InSocketWaitTime)
{
	while (!PrioritySendQueue.IsEmpty() || !SendQueue.IsEmpty())
	{
		if (!Socket->Wait(ESocketWaitConditions::WaitForWrite, InSocketWaitTime))
		{
			return EUpdateResult::Retry;
		}

		SCOPED_MESSAGING_TRACE(UdpMessaging_FSocketSender_Update);

		const bool bIsPriority = !PrioritySendQueue.IsEmpty();
		TMpscQueue<FPacket>& Queue = bIsPriority ? PrioritySendQueue : SendQueue;

		const FPacket* Packet = Queue.Peek();
		const FIPv4Endpoint Recipient = Packet->Recipient;
		const int32 PacketNumBytes = Packet->Data->Num();

		const int32 TokenBytes = PacketNumBytes + PacketHeaderBytes;
		const bool bConsumedTokens = TokenBucket.TryConsume(TokenBytes);
		if (!bConsumedTokens && !bIsPriority)
		{
			if (!ensure(TokenBytes <= TokenBucket.Max()))
			{
				UE_LOGFMT(LogUdpMessaging, Error,
					"Packet size {TokenBytes} exceeds max burst size {MaxBytes}",
					TokenBytes, TokenBucket.Max());

				return EUpdateResult::Fatal;
			}

			return EUpdateResult::Retry;
		}

		int32 SentNumBytes = 0;
		const bool bSendResult =
			Socket->SendTo(Packet->Data->GetData(), PacketNumBytes, SentNumBytes, *Recipient.ToInternetAddr());

		ESocketErrors SocketError = SE_NO_ERROR;

		if (!bSendResult)
		{
			if (bConsumedTokens)
			{
				// Refund tokens that never actually egressed.
				TokenBucket.Credit(TokenBytes);
			}

			SocketError = SocketSubsystem->GetLastErrorCode();
			if (SocketError == SE_EWOULDBLOCK || SocketError == SE_ENOBUFS || SocketError == SE_EINTR)
			{
				// Leave the packet in the queue to try again.
				return EUpdateResult::Retry;
			}
		}

		// Only pop after we've checked for "retry-able" error codes.
		Packet = nullptr;
		Queue.Dequeue();

		if (!bSendResult)
		{
			bool bFatalError;
			switch (SocketError)
			{
				case SE_ENETDOWN:
				case SE_ENETUNREACH:
				case SE_EHOSTDOWN:
				case SE_EHOSTUNREACH:
				case SE_EADDRNOTAVAIL:
					bFatalError = false;
					break;
				default:
					bFatalError = true;
					break;
			}

			if (!bFatalError)
			{
				// Non-fatal; on to the next packet in the queue.
				UE_LOGFMT(LogUdpMessaging, Verbose,
					"Sender {Description}: SendTo failed (destination: {Recipient}) ({ErrorStr})",
					Description, Recipient.ToString(), SocketSubsystem->GetSocketError(SocketError));
				continue;
			}
			else
			{
				UE_LOGFMT(LogUdpMessaging, Error,
					"Sender {Description}: SendTo failed (destination: {Recipient}) ({ErrorStr})",
					Description, Recipient.ToString(), SocketSubsystem->GetSocketError(SocketError));
			}
		}

		if (SentNumBytes != PacketNumBytes)
		{
			if (SentNumBytes >= 0)
			{
				// FIXME?: In the absence of another socket error, could be a retry? Is this even possible for UDP?
				UE_LOGFMT(LogUdpMessaging, Error,
					"Sender {Description}: Incomplete send (destination: {Recipient}) ({Sent}/{Total} bytes)",
					Description, Recipient.ToString(), SentNumBytes, PacketNumBytes);
			}
			return EUpdateResult::Fatal;
		}
	}

	return EUpdateResult::Done;
}


void FSocketSender::TryEnableSegmentationOffload(uint16 InSegmentOffloadSize)
{
	UE_LOGFMT(LogUdpMessaging, Log, "Sender {Description}: Attempting to enable segmentation offload", Description);

	if (Socket->SetSendSegmentationOffloadSize(InSegmentOffloadSize))
	{
		UE_LOGFMT(LogUdpMessaging, Log, "Sender {Description}: Segmentation offload enabled ({SegmentSize} bytes)",
			Description, InSegmentOffloadSize);

		SegmentOffloadSize = InSegmentOffloadSize;
	}
	else
	{
		const ESocketErrors SocketError = SocketSubsystem->GetLastErrorCode();
		UE_LOGFMT(LogUdpMessaging, Warning, "Sender {Description}: Failed to enable segmentation offload ({Error})",
			Description, SocketSubsystem->GetSocketError());

		SegmentOffloadSize = 0;
	}
}


} // namespace UE::UdpMessaging
