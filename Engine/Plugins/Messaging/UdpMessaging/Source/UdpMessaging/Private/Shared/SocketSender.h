// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>
#include "Containers/MpscQueue.h"
#include "HAL/Runnable.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/SingleThreadRunnable.h"
#include "Shared/TokenBucket.h"
#include "Templates/SharedPointer.h"


class FEvent;
class FInternetAddr;
class FRunnableThread;
class FSocket;
class ISocketSubsystem;


namespace UE::UdpMessaging {

/**
 * Asynchronously sends data to a UDP socket.
 */
class FSocketSender
	: public FRunnable
	, private FSingleThreadRunnable
{
	// Structure for outbound packets.
	struct FPacket
	{
		/** Holds the packet's data. */
		TSharedPtr<TArray<uint8>> Data;

		/** Holds the recipient. */
		FIPv4Endpoint Recipient;

		/** Default constructor. */
		FPacket() { }

		/** Creates and initializes a new instance. */
		FPacket(const TSharedRef<TArray<uint8>>& InData, const FIPv4Endpoint& InRecipient)
			: Data(InData)
			, Recipient(InRecipient)
		{ }
	};

	enum class EUpdateResult
	{
		/** The queued was emptied, and we can await new work. */
		Done,

		/** There was a transient failure, and we should attempt to service the queue again soon. */
		Retry,

		/** Tear down this FSocketSender and percolate the failure to the caller. */
		Fatal,
	};

public:
	struct FOptions
	{
		int32 SendBufferSize = 512 * 1024;
		FTimespan WaitTime = FTimespan::FromMilliseconds(100);

		/** Datagram payload size for UDP segmentation offload; 0 disables USO. Ignored if unsupported. */
		uint16 SegmentOffloadSize = 0;

		/** Token bucket: steady state send bytes per second / replenishment rate. */
		int64 BytesPerSec = 100 * 1024 * 1024;

		/** Token bucket: max bucket reserve capacity. */
		int64 MaxBurstBytes = 10 * 1024 * 1024;

		FOptions()
		{
			// Without this (or with `= default`), clang gives the following:
			// `error: default member initializer for 'SendBufferSize' needed within definition
			// of enclosing class 'FSocketSender' outside of member functions` (et cetera)
		}
	};

	/**
	 * Creates and initializes a new socket sender.
	 *
	 * @param InSocket The UDP socket to use for sending data.
	 * @param InDescription The description text (for debugging).
	 * @param InOptions Other configuration.
	 */
	FSocketSender(FSocket* InSocket, const TCHAR* InDescription, const FOptions& InOptions = FOptions());

	/** Virtual destructor. */
	virtual ~FSocketSender();

	/**
	 * Sends data to the specified recipient.
	 *
	 * @param InData The data to send.
	 * @param InRecipient The recipient.
	 * @param bAllowSegmentation Pass true if you are intentionally sending more than `GetSegmentOffloadSize()` bytes.
	 * @param bHighPriority Packets in this queue are always sent first.
	 * @return true if the data will be sent, false otherwise.
	 */
	bool Send(
		const TSharedRef<TArray<uint8>>& InData,
		const FIPv4Endpoint& InRecipient,
		bool bAllowSegmentation = false,
		bool bHighPriority = false
	);

	/** Get the address to which this socket is bound. */
	const TSharedRef<FInternetAddr>& GetAddress() const { return SocketAddr; }

	/** If send segmentation offload is enabled, returns the max datagram payload. Otherwise 0. */
	uint16 GetSegmentOffloadSize() const { return SegmentOffloadSize; }

	//~ FRunnable interface
	virtual FSingleThreadRunnable* GetSingleThreadInterface() override
	{
		return this;
	}

	virtual bool Init() override
	{
		return true;
	}

	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override { }

	//~ FSingleThreadRunnable interface
	virtual void Tick() override
	{
		Update(FTimespan::Zero());
	}

protected:
	/**
	 * Update this socket sender.
	 *
	 * @param InSocketWaitTime Time to wait for socket write conditions.
	 */
	EUpdateResult Update(const FTimespan& InSocketWaitTime);

	void TryEnableSegmentationOffload(uint16 InSegmentOffloadSize);

private:
	/** The send queue. */
	TMpscQueue<FPacket> SendQueue;

	/** High-priority send queue. */
	TMpscQueue<FPacket> PrioritySendQueue;

	/** The network socket. */
	FSocket* Socket;

	/** Cached pointer to the platform socket subsystem. */
	ISocketSubsystem* SocketSubsystem;

	/** Cached socket binding address. */
	TSharedRef<FInternetAddr> SocketAddr;

	/** Flag indicating that the thread is stopping. */
	std::atomic<bool> bStopping;

	/** The thread object. */
	FRunnableThread* Thread;

	/** Maximum time to wait for the socket to be ready to send. */
	FTimespan WaitTime;

	/** An event signaling that outbound messages need to be processed. */
	FEvent* WorkEvent;

	/** Description of this sender (for debugging). */
	FString Description;

	/** Datagram payload size for segmentation offload. 0 means disabled or unsupported. */
	uint16 SegmentOffloadSize;

	/** Token bucket for egress shaping. */
	FLockFreeTokenBucket TokenBucket;
};

} // namespace UE::UdpMessaging
