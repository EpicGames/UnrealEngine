// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "HAL/PlatformTime.h"

#include <atomic>


namespace UE::UdpMessaging
{

class FLockFreeTokenBucket
{
public:
	FLockFreeTokenBucket(int64 InTokensPerSec, int64 InMaxBurstTokens)
		: TokensPerSec(InTokensPerSec)
		, MaxBurstTokens(InMaxBurstTokens)
	{
		check(TokensPerSec > 0);
		check(MaxBurstTokens > 0);

		FullAtTimeCycles = FPlatformTime::Cycles64();
	}

	int64 Max() const
	{
		return MaxBurstTokens;
	}

	bool TryConsume(int64 InConsumeTokens)
	{
		check(InConsumeTokens > 0);

		if (UNLIKELY(!ensure(InConsumeTokens <= MaxBurstTokens)))
		{
			return false;
		}

		const double SecondsPerCycle64 = FPlatformTime::GetSecondsPerCycle64();
		int64 Expected = FullAtTimeCycles.load(std::memory_order_relaxed);
		int64 Desired;
		do 
		{
			const int64 NowCycles = FPlatformTime::Cycles64();
			const double SecToFull = (Expected - NowCycles) * SecondsPerCycle64;
			const int64 TokensAvailable = FMath::Clamp(MaxBurstTokens - SecToFull * TokensPerSec, 0ll, MaxBurstTokens);
			if (InConsumeTokens > TokensAvailable)
			{
				return false;
			}

			const double ConsumedSec = InConsumeTokens / static_cast<double>(TokensPerSec);
			const int64 ConsumedCycles = FMath::CeilToInt64(ConsumedSec / SecondsPerCycle64);
			Desired = FMath::Max(Expected, NowCycles) + ConsumedCycles;
		} while (!FullAtTimeCycles.compare_exchange_weak(Expected, Desired));

		return true;
	}

	void Credit(int64 InCreditTokens)
	{
		check(InCreditTokens > 0);

		const double SecondsPerCycle64 = FPlatformTime::GetSecondsPerCycle64();
		const double CreditSec = InCreditTokens / static_cast<double>(TokensPerSec);
		const int64 CreditCycles = FMath::CeilToInt64(CreditSec / SecondsPerCycle64);
		FullAtTimeCycles.fetch_sub(CreditCycles);
	}

private:
	const int64 TokensPerSec;
	const int64 MaxBurstTokens;

	/** A single atomic value that represents the current state. If <= now, the bucket is full. */
	std::atomic<int64> FullAtTimeCycles;
};

} // namespace UE::UdpMessaging
