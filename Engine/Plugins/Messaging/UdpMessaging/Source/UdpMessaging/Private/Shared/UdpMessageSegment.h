// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

#include "Algo/Transform.h"

enum class EMessageFlags : uint32;

/**
 * Enumerates message segment types.
 */
enum class EUdpMessageSegments : uint8
{
	/** None. */
	None,

	/** Request to abort the sending of a message. */
	Abort,

	/** Acknowledges that the message was received successfully. */
	Acknowledge,

	/** Notifies the bus that an endpoint has left. */
	Bye,

	/** A message data segment. */
	Data,

	/** Notifies the bus that an endpoint has joined. */
	Hello,

	/** Request to retransmit selected data segments. */
	Retransmit,

	/** Notification that an inbound message timed out. */
	Timeout,

	/** Acknowledges that message segments were received successfully */
	AcknowledgeSegments,

	/** Announces existence to static endpoints. */
	Ping,

	/** Answers back to ping segment. */
	Pong,

	/** Used to share endpoints for fully meshing networks. */
	Mesh

	// New segment type needs to be added at the end
};


namespace FUdpMessageSegment
{
	/**
	 * Structure for the header of all segments.
	 */
	struct FHeader
	{
		/** Holds the protocol version. */
		uint8 ProtocolVersion;

		/** Holds the recipient's node identifier (empty = multicast). */
		FGuid RecipientNodeId;

		/** Holds the sender's node identifier. */
		FGuid SenderNodeId;

		/**
		 * Holds the segment type.
		 *
		 * @see EUdpMessageSegments
		 */
		EUdpMessageSegments SegmentType;

		static constexpr uint32 SerializedBytes = (
			sizeof(ProtocolVersion) +
			sizeof(RecipientNodeId) +
			sizeof(SenderNodeId) +
			sizeof(SegmentType)
		);

	public:

		/**
		 * Serializes the given header from or into the specified archive.
		 *
		 * @param Ar The archive to serialize from or into.
		 * @param Header The header to serialize.
		 * @return The archive.
		 */
		friend FArchive& operator<<(FArchive& Ar, FHeader& Header)
		{
			return Ar << Header.ProtocolVersion << Header.RecipientNodeId << Header.SenderNodeId << Header.SegmentType;
		}
	};

	/**
	 * Structure for the sub-header of Abort segments.
	 */
	struct FAbortChunk
	{
		/** Holds the identifier of the message to abort. */
		int32 MessageId;

	public:

		/**
		 * Serializes the given header from or to the specified archive for the specified version.
		 *
		 * @param Ar The archive to serialize from or into.
		 * @param ProtocolVersion The protocol version we want to serialize the Chunk in.
		 */
		void Serialize(FArchive& Ar, uint8 /*ProtocolVersion*/)
		{
			Ar << MessageId;
		}
	};


	/**
	 * Structure for the header of Acknowledge segments.
	 */
	struct FAcknowledgeChunk
	{
		/** Holds the identifier of the message that was received successfully. */
		int32 MessageId;

	public:
		/**
		 * Serializes the given header from or to the specified archive for the specified version.
		 *
		 * @param Ar The archive to serialize from or into.
		 * @param ProtocolVersion The protocol version we want to serialize the Chunk in.
		 */
		void Serialize(FArchive& Ar, uint8 /*ProtocolVersion*/)
		{
			Ar << MessageId;
		}
	};


	/**
	 * Structure for the header of AcknowledgeSegments segments.
	 */
	struct FAcknowledgeSegmentsChunk
	{
	private:
		// Previous version serialization struct
		struct FV10_11
		{
			int32 MessageId;
			TArray<uint16> Segments;
			FAcknowledgeSegmentsChunk& Chunk; // Reference for loading

			FV10_11(FAcknowledgeSegmentsChunk& InChunk)
				: MessageId(InChunk.MessageId)
				, Chunk(InChunk)
			{
				Algo::Transform(Chunk.Segments, Segments, [](uint32 Value)
				{
					return (uint16)Value;
				});
			}

			void Serialize(FArchive& Ar, uint8)
			{
				Ar	<< MessageId
					<< Segments;
				// Copy back if loading
				if (Ar.IsLoading())
				{
					Chunk.MessageId = MessageId;
					Algo::Transform(Segments, Chunk.Segments, [](uint16 Value)
					{
						return (uint32)Value;
					});
				}
			}
		};

	public:
		/** Holds the identifier of the message that received segments successfully. */
		int32 MessageId;

		/** List of Acknowledged segments */
		TArray<uint32> Segments;

		static constexpr uint32 FixedSerializedBytes = (
			sizeof(MessageId) +
			sizeof(decltype(Segments)::SizeType)
		);

	public:
		FAcknowledgeSegmentsChunk() = default;

		FAcknowledgeSegmentsChunk(int32 InMessageId, TArray<uint32>&& InSegments)
			: MessageId(InMessageId)
			, Segments(MoveTemp(InSegments))
		{}

		/**
		 * Serializes the given header from or to the specified archive for the specified version.
		 *
		 * @param Ar The archive to serialize from or into.
		 * @param ProtocolVersion The protocol version we want to serialize the Chunk in.
		 */
		void Serialize(FArchive& Ar, uint8 ProtocolVersion)
		{
			// version 12 and onward (current version)
			if (ProtocolVersion > 11)
			{
				Ar	<< MessageId
					<< Segments;
			}
			// if previous version 10 or 11
			else
			{
				FV10_11 PreviousV10_11(*this);
				PreviousV10_11.Serialize(Ar, ProtocolVersion);
			}
		}
	};


	/**
	 * Structure for the header of Data segments.
	 */
	struct FDataChunk
	{
	private:
		// Previous version serialization struct
		struct FV10_11
		{
			int32 MessageId;
			int32 MessageSize;
			uint16 SegmentNumber;
			uint32 SegmentOffset;
			uint64 Sequence;
			uint16 TotalSegments;
			//TArray<uint8> Data;		// Taken from the actual chunk, listed here for reference
			EMessageFlags MessageFlags;
			FDataChunk& Chunk;			// Reference for loading


			FV10_11(FDataChunk& InChunk)
				: MessageId(InChunk.MessageId)
				, MessageSize((int32)InChunk.MessageSize)
				, SegmentNumber((uint16)InChunk.SegmentNumber)
				, SegmentOffset((uint32)InChunk.SegmentOffset)
				, Sequence(InChunk.Sequence)
				, TotalSegments((uint16)InChunk.TotalSegments)
				, MessageFlags(InChunk.MessageFlags)
				, Chunk(InChunk)
			{}

			void Serialize(FArchive& Ar, uint8 ProtocolVersion)
			{
				Ar	<< MessageId
					<< MessageSize
					<< SegmentNumber
					<< SegmentOffset
					<< Sequence
					<< TotalSegments
					<< Chunk.Data;
				if (ProtocolVersion > 10)
				{
					Ar << MessageFlags;
				}
				// Copy back to Chunk, Data is directly deserialized in the Chunk
				if (Ar.IsLoading())
				{
					Chunk.MessageId = MessageId;
					Chunk.MessageSize = MessageSize;
					Chunk.MessageFlags = MessageFlags;
					Chunk.SegmentNumber = SegmentNumber;
					Chunk.SegmentOffset = SegmentOffset;
					Chunk.TotalSegments = TotalSegments;
					Chunk.Sequence = Sequence;
				}
			}
		};

	public:
		/** Holds the identifier of the message that the data belongs to. */
		int32 MessageId;

		/** Holds the total size of the message. */
		int64 MessageSize;

		/** Holds the message flags. */
		EMessageFlags MessageFlags;

		/** Holds the sequence number of this segment. */
		uint32 SegmentNumber;

		/** Holds the segment's offset within the message. */
		uint64 SegmentOffset;

		/** Holds the total number of data segments being sent. */
		uint32 TotalSegments;

		/** Holds the message sequence number (0 = not sequential). */
		uint64 Sequence;

		/** Holds the segment data. */
		TArray<uint8> Data;

		static constexpr uint32 FixedSerializedBytes = (
			sizeof(MessageId) +
			sizeof(MessageSize) +
			sizeof(MessageFlags) +
			sizeof(SegmentNumber) +
			sizeof(SegmentOffset) +
			sizeof(TotalSegments) +
			sizeof(Sequence) +
			sizeof(decltype(Data)::SizeType)
		);

	public:
		FDataChunk() = default;

		/**
		 * Serializes the given header from or to the specified archive for the specified version.
		 *
		 * @param Ar The archive to serialize from or into.
		 * @param ProtocolVersion The protocol version we want to serialize the Chunk in.
		 */
		void Serialize(FArchive& Ar, uint8 ProtocolVersion)
		{
			// version 12 and onward (current version)
			if (ProtocolVersion > 11)
			{
				Ar	<< MessageId
					<< MessageSize
					<< MessageFlags
					<< SegmentNumber
					<< SegmentOffset
					<< TotalSegments
					<< Sequence
					<< Data;
			}
			// if previous version 10 or 11
			else
			{
				FV10_11 PreviousV10_11(*this);
				PreviousV10_11.Serialize(Ar, ProtocolVersion);
			}
		}
	};


	/**
	 * Structure for the sub-header of Retransmit segments.
	 *
	 * Retransmit segments are sent from a message recipient to a message sender in order
	 * to indicate that selected message segments are to be retransmitted, i.e. if they
	 * got lost on the network or if the recipient was unable to handle them previously.
	 */
	struct FRetransmitChunk
	{
		/** Holds the identifier of the message for which data needs to be retransmitted. */
		int32 MessageId;

		/** Holds the list of data segments that need to be retransmitted. */
		TArray<uint16> Segments;

	public:

		/**
		 * Serializes the given header from or to the specified archive for the specified version.
		 *
		 * @param Ar The archive to serialize from or into.
		 * @param ProtocolVersion The protocol version we want to serialize the Chunk in.
		 */
		void Serialize(FArchive& Ar, uint8 /*ProtocolVersion*/)
		{
			Ar << MessageId << Segments;
		}

		int32 GetMessageId(FArchive& Ar)
		{
			int32 Id;
			Ar << Id;
			return Id;
		}
	};


	/**
	 * Structure for the header of Timeout packets.
	 */
	struct FTimeoutChunk
	{
		/** Holds the identifier of the message that timed out. */
		int32 MessageId;

	public:

		/**
		 * Serializes the given header from or to the specified archive for the specified version.
		 *
		 * @param Ar The archive to serialize from or into.
		 * @param ProtocolVersion The protocol version we want to serialize the Chunk in.
		 */
		void Serialize(FArchive& Ar, uint8 /*ProtocolVersion*/)
		{
			Ar << MessageId;
		}
	};
};
