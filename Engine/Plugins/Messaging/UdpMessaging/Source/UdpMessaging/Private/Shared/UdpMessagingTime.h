// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/PlatformTime.h"
#include "Misc/DateTime.h"
#include "Misc/Timespan.h"


/** Thin wrapper to help migrate from FDateTime to FPlatformTime for relative measurements. */
struct FUdpMessagingTime
{
	int64 Cycles = 0;

	static FUdpMessagingTime Now()
	{
		return { .Cycles = static_cast<int64>(FPlatformTime::Cycles64()) };
	}

	FDateTime EstimatedLocalDateTime() const
	{
		const FUdpMessagingTime RelNow = Now();
		const FDateTime AbsNow = FDateTime::Now();
		return AbsNow - (RelNow - *this);
	}

	FTimespan operator-(FUdpMessagingTime Other) const
	{
		return FTimespan::FromSeconds((Cycles - Other.Cycles) * FPlatformTime::GetSecondsPerCycle64());
	}

	auto operator+(FUdpMessagingTime Other) const = delete;
	auto operator+=(FUdpMessagingTime Other) const = delete;

	FUdpMessagingTime operator-(FTimespan Other) const
	{
		const int64 Delta = FMath::RoundToInt64(Other.GetTotalSeconds() / FPlatformTime::GetSecondsPerCycle64());
		return { .Cycles = Cycles - Delta };
	}

	FUdpMessagingTime operator+(FTimespan Other) const
	{
		const int64 Delta = FMath::RoundToInt64(Other.GetTotalSeconds() / FPlatformTime::GetSecondsPerCycle64());
		return { .Cycles = Cycles + Delta };
	}

#if 0 // Enable some day when all our target toolchains support it.
	/** Consistent compiler generated default comparison operations based on Cycles. */
	friend auto operator<=>(FUdpMessagingTime, FUdpMessagingTime) = default;
#else
	bool operator==(FUdpMessagingTime Other) const { return Cycles == Other.Cycles; }
	bool operator!=(FUdpMessagingTime Other) const { return Cycles != Other.Cycles; }
	bool operator>(FUdpMessagingTime Other) const { return Cycles > Other.Cycles; }
	bool operator>=(FUdpMessagingTime Other) const { return Cycles >= Other.Cycles; }
	bool operator<(FUdpMessagingTime Other) const { return Cycles < Other.Cycles; }
	bool operator<=(FUdpMessagingTime Other) const { return Cycles <= Other.Cycles; }
#endif
};
