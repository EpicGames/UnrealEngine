// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "Containers/ArrayBuilder.h"

#include "UdpMessagingPrivate.h"
#include "Shared/UdpMessagingSettings.h"
#include "Transport/UdpSerializedMessage.h"
#include "Transport/UdpMessageSegmenter.h"


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUdpMessageSegmenterTest, "System.Core.Messaging.Transports.Udp.UdpMessageSegmenter", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)


void RunSegmentationTest(FAutomationTestBase& Test, uint32 MessageSize, uint16 SegmentPayloadSize, bool WithReliableMessages)
{
	check(MessageSize < (uint32)255 * SegmentPayloadSize);

	uint16 NumSegments = (MessageSize + SegmentPayloadSize - 1) / SegmentPayloadSize;
	Test.AddInfo(FString::Printf(TEXT("Segmenting message of size %i with %i payloads of size %i (Reliable=%d)..."),
		MessageSize, NumSegments, SegmentPayloadSize, WithReliableMessages));

	// create a large message to segment
	TSharedRef<FUdpSerializedMessage> Message = MakeShared<FUdpSerializedMessage>(
		EUdpMessageFormat::CborPlatformEndianness, UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION,
		WithReliableMessages ? EMessageFlags::Reliable : EMessageFlags::None);

	for (uint8 SegmentIndex = 0; SegmentIndex < NumSegments; ++SegmentIndex)
	{
		// write segment index into each byte of the current segment
		for (uint16 Offset = 0; (Offset < SegmentPayloadSize) && (Message->TotalSize() < MessageSize); ++Offset)
		{
			*Message << SegmentIndex;
		}
	}

	Message->UpdateState(EUdpSerializedMessageState::Complete);

	// create and initialize segmenter
	const uint16 SegmentSize = IntCastChecked<uint16>((uint32)SegmentPayloadSize + FUdpMessageSegmenter::GetDataSegmentHeaderSize());
	FUdpMessageSegmenter Segmenter = FUdpMessageSegmenter(Message, SegmentSize);

	const bool bInitializeResult = Segmenter.TryInitialize();
	Test.TestTrue(TEXT("Attempting to initialize the segmenter succeeds"), bInitializeResult);

	Test.TestEqual(TEXT("Our calculation of segment count must match the segmenter"), Segmenter.GetSegmentCount(), NumSegments);

	// invariants
	{
		Test.TestEqual(TEXT("The message size must match the actual message size"), Segmenter.GetMessageSize(), Message->TotalSize());
		Test.TestEqual(TEXT("The total number of segments must match the actual number of segments in the message"), Segmenter.GetSegmentCount(), (uint32)NumSegments);
	}

	// pre-conditions
	{
		Test.TestEqual(TEXT("The initial number of pending segments must match the total number of segments in the message"), Segmenter.GetPendingSendSegmentsCount(), (uint32)NumSegments);
		Test.TestFalse(TEXT("Segmentation of a non-empty message must not be complete initially"), Segmenter.IsSendingComplete());
	}

	// peeking at next pending segment
	{
		TArray<uint8> OutData;
		uint32 OutSegmentNumber;

		Segmenter.GetNextPendingSegment(OutData, OutSegmentNumber);

		Test.TestEqual(TEXT("The number of pending segments must not change when peeking at a pending segment"), Segmenter.GetPendingSendSegmentsCount(), (uint32)NumSegments);
	}

	uint32 GeneratedSegmentCount = 0;

	// do the segmentation
	{
		TArray<uint8> OutData;
		uint32 OutSegmentNumber = 0;
		uint32 NumInvalidSegments = 0;
		uint32 NumSends = 0;

		while (Segmenter.GetNextPendingSegment(OutData, OutSegmentNumber))
		{
			Test.TestFalse(TEXT("IsSendingComplete should return false for the segment until all segments are sent"), Segmenter.IsSendingComplete());

			// mark as sent and check total
			Segmenter.MarkAsSent(TArrayBuilder<uint32>().Add(OutSegmentNumber));
			NumSends++;

			// test how many are remaining
			Test.TestEqual(TEXT("The number of pending send segments must be the total count minus those sent"), Segmenter.GetPendingSendSegmentsCount(), NumSegments - NumSends);

			Test.TestEqual(TEXT("The number of bits in the pending segment array must match the returned count"), Segmenter.GetPendingSendSegments().CountSetBits(), Segmenter.GetPendingSendSegmentsCount());
			
			// Test reliable state
			if (WithReliableMessages)
			{
				Test.TestEqual(TEXT("The number of acknowledged segments should be 0 after sending"), Segmenter.GetAcknowledgedSegmentsCount(), 0);
				Test.TestEqual(TEXT("The number of bits in the pending acknowledgement array must be zero"), Segmenter.GetAcknowledgedSegments().CountSetBits(), 0);
				Test.TestFalse(TEXT("The message should have outstanding acknowledgements"), Segmenter.AreAcknowledgementsComplete());
			}
			else
			{
				Test.TestEqual(TEXT("Unreliable messages should not have any received ack segments"), Segmenter.GetAcknowledgedSegmentsCount(), 0);
				Test.TestEqual(TEXT("The bits in the pending acknowledgement array must be zero for unreliable messages"), Segmenter.GetAcknowledgedSegments().CountSetBits(), 0);
				Test.TestTrue(TEXT("Unreliable messages should always be marked as ack's being complete"), Segmenter.AreAcknowledgementsComplete());
			}

			++GeneratedSegmentCount;

			// verify segment size
			int32 ExpectedSegmentSize = (OutSegmentNumber == (NumSegments - 1))
				? MessageSize % SegmentPayloadSize
				: SegmentPayloadSize;

			if (ExpectedSegmentSize == 0)
			{
				ExpectedSegmentSize = SegmentPayloadSize;
			}

			if (OutData.Num() != ExpectedSegmentSize)
			{
				++NumInvalidSegments;

				continue;
			}

			// verify segment data
			for (const auto& Element : OutData)
			{
				if (OutSegmentNumber != Element)
				{
					++NumInvalidSegments;
				}
			}
		}

		Test.TestTrue(TEXT("IsSendingComplete should return true after all segments are sent"), Segmenter.IsSendingComplete());

		if (WithReliableMessages)
		{
			Test.TestFalse(TEXT("AreAcknowledgementsComplete should return false after all segments for a reliable message are sent"), Segmenter.AreAcknowledgementsComplete());

			// Simulate what would happen if we decided to retransmit a packet but then received our ack before the send occurred...

			// mark a packet for retransmission
			auto SegmentsToRetransmit = TArrayBuilder<uint16>().Add(OutSegmentNumber);
			Segmenter.MarkForRetransmission(SegmentsToRetransmit);
			Test.TestFalse(TEXT("Marking an outstanding segment for restransmission should result in sending being incomplete"), Segmenter.IsSendingComplete());

			// Mark packets as acknowledged
			uint32 NumAckd = 0;
			for (uint32 i = 0; i < NumSends; i++)
			{
				Segmenter.MarkAsAcknowledged(TArrayBuilder<uint32>().Add(NumAckd++));
				Test.TestEqual(TEXT("The number of bits in the pending acknowledgement array must match the internal count"), Segmenter.GetAcknowledgedSegments().CountSetBits(), Segmenter.GetAcknowledgedSegmentsCount());
			}

			// Send a packet that has already been acknowledged
			auto SegmentsToSend = TArrayBuilder<uint32>().Add(OutSegmentNumber);
			Segmenter.MarkAsSent(SegmentsToSend);
			Test.TestTrue(TEXT("Retransmitting an outstanding segment should result in sending being complete"), Segmenter.IsSendingComplete());

			// Even though we marked it for retransmission, the segmenter should report acks are complete
			Test.TestTrue(TEXT("MarkForRetransmission should not cause the state of an acknowledged packet to change"), Segmenter.AreAcknowledgementsComplete());
		}
		else
		{
			Test.TestTrue(TEXT("AreAcknowledgementsComplete should always be true for an unreliable message"), Segmenter.AreAcknowledgementsComplete());
		}
		

		Test.TestEqual(TEXT("The number of generated segments must match the total number of segments in the message"), GeneratedSegmentCount, (uint32)NumSegments);
		Test.TestEqual(TEXT("The number of invalid segments must be zero"), NumInvalidSegments, 0u);
	}

	// post-conditions
	{
		Test.TestEqual(TEXT("The number of pending segments must be zero after segmentation is complete"), Segmenter.GetPendingSendSegmentsCount(), 0u);
		Test.TestTrue(TEXT("Segmentation must be complete when there are no more pending segments"), Segmenter.IsSendingComplete());
	}
}


bool FUdpMessageSegmenterTest::RunTest(const FString& Parameters)
{
	bool ReliableOptions[] = {false, true};

	for(bool WithReliable : ReliableOptions)
	{
		// single partial segment
		RunSegmentationTest(*this, 123, 1024, WithReliable);

		// single full segment
		RunSegmentationTest(*this, 1024, 1024, WithReliable);

		// multiple segments with partial last segment
		RunSegmentationTest(*this, 100000, 1024, WithReliable);

		// multiple segments without partial last segment
		RunSegmentationTest(*this, 131072, 1024, WithReliable);
	}

	return true;
}


void EmptyLinkFunctionForStaticInitializationUdpMessageSegmenterTest()
{
	// This function exists to prevent the object file containing this test from
	// being excluded by the linker, because it has no publicly referenced symbols.
}
