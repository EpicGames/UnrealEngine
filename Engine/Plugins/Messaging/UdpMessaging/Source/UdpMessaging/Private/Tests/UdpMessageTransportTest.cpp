// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"
#include "IMessageTransportHandler.h"
#include "Templates/SharedPointerInternals.h"
#include "Transport/UdpMessageTransport.h"
#include "Tests/UdpMessagingTestTypes.h"
#include "MessageEndpoint.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUdpMessageTransportTest, "System.Core.Messaging.Transports.Udp.UdpMessageTransport", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

class FUdpMessageTransportTestState
	: public IMessageTransportHandler
{
public:

	FUdpMessageTransportTestState(FAutomationTestBase& InTest, const FIPv4Endpoint& UnicastEndpoint, const FIPv4Endpoint& MulticastEndpoint, uint8 MulticastTimeToLive)
		: NumReceivedMessages(0)
	{
		TArray<FIPv4Endpoint> StaticEndpoints;
		TArray<FString> ExcludeEndpoints;
		Transport = MakeShared<FUdpMessageTransport, ESPMode::ThreadSafe>(UnicastEndpoint, MulticastEndpoint,
																		  MoveTemp(StaticEndpoints),
																		  MoveTemp(ExcludeEndpoints), MulticastTimeToLive);
	}

public:

	const TArray<FGuid>& GetDiscoveredNodes() const
	{
		return DiscoveredNodes;
	}

	const TArray<FGuid>& GetLostNodes() const
	{
		return LostNodes;
	}

	int32 GetNumReceivedMessages() const
	{
		return NumReceivedMessages;
	}

	bool Publish(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context)
	{
		return Transport->TransportMessage(Context, TArray<FGuid>());
	}

	bool Start()
	{
		return Transport->StartTransport(*this);
	}

	void Stop()
	{
		Transport->StopTransport();
	}

public:

	//~ IMessageTransportHandler interface

	virtual void DiscoverTransportNode(const FGuid& NodeId) override
	{
		DiscoveredNodes.Add(NodeId);
	}

	virtual void ForgetTransportNode(const FGuid& NodeId) override
	{
		LostNodes.Add(NodeId);
	}

	virtual void ReceiveTransportMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const FGuid& NodeId) override
	{
		FPlatformAtomics::InterlockedIncrement(&NumReceivedMessages);
		FScopeLock LastMessageContextScopeLock(&LastMessageContextLock);
		LastMessageContext = Context;
	}

	TSharedPtr<IMessageContext, ESPMode::ThreadSafe> GetLastMessage() const
	{
		FScopeLock LastMessageContextScopeLock(&LastMessageContextLock);
		return LastMessageContext;
	}

private:

	void TestWildcardEndpointExclusion();

	TArray<FGuid> DiscoveredNodes;
	TArray<FGuid> LostNodes;
	int32 NumReceivedMessages;
	mutable FCriticalSection LastMessageContextLock;
	TSharedPtr<IMessageTransport, ESPMode::ThreadSafe> Transport;
	TSharedPtr<IMessageContext, ESPMode::ThreadSafe> LastMessageContext;
};

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUdpMessageTransportTestExclusionList, "System.Core.Messaging.Transports.Udp.UdpMessageTransportExclusionList", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)

bool FUdpMessageTransportTestExclusionList::RunTest(const FString& Parameters)
{
	const FIPv4Endpoint SimpleEndpoint(FIPv4Address(192, 168, 10, 120), 12777);

	TArray<FString> PassTests = {
		TEXT("192.16?.10.*"),
		TEXT("192.168.10.120"),
		TEXT("192.168.10.*:0"),
		TEXT("192.168.10.120:12777"),
		TEXT("*.*.*.*:12777")
	};
	for (const FString& Test : PassTests)
	{
		AddInfo(FString::Printf(TEXT("Testing wildcard string %s against target %s"), *Test, *SimpleEndpoint.ToString()));
		TestTrue(TEXT("Wilcard match should succeed."),	 FUdpMessageTransport::MatchesEndpoint(Test, SimpleEndpoint));
	}

	TArray<FString> FailTests = {
		TEXT("192.168.10.11"),
		TEXT("192.168.10.120:1210"),
		TEXT("192.168.*.55:0"),
		TEXT("192.168.11.11")
    };

	for (const FString& Test : FailTests)
	{
		AddInfo(FString::Printf(TEXT("Testing wildcard string %s against target %s"), *Test, *SimpleEndpoint.ToString()));
		TestFalse(TEXT("Wilcard match should not succeed."), FUdpMessageTransport::MatchesEndpoint(Test, SimpleEndpoint));
	}
	return true;
}

bool FUdpMessageTransportTest::RunTest(const FString& Parameters)
{
	const FIPv4Endpoint MulticastEndpoint(FIPv4Address(231, 0, 0, 1), 7777);
	const FIPv4Endpoint UnicastEndpoint = FIPv4Endpoint::Any;
	const uint8 MulticastTimeToLive = 0;
	const int32 NumTestMessages = 10000;
	const int32 MessageSize = 1280;

	// create transports
	FUdpMessageTransportTestState Transport1(*this, UnicastEndpoint, MulticastEndpoint, MulticastTimeToLive);
	const auto& DiscoveredNodes1 = Transport1.GetDiscoveredNodes();
	
	FUdpMessageTransportTestState Transport2(*this, UnicastEndpoint, MulticastEndpoint, MulticastTimeToLive);
	const auto& DiscoveredNodes2 = Transport2.GetDiscoveredNodes();

	// test transport node discovery
	{
		Transport1.Start();
		FPlatformProcess::Sleep(3.0f);

		TestTrue(TEXT("A single message transport must not discover any remote nodes"), DiscoveredNodes1.Num() == 0);

		Transport2.Start();
		FPlatformProcess::Sleep(3.0f);

		if (DiscoveredNodes1.Num() == 0)
		{
			AddError(TEXT("The first transport did not discover any nodes"));
			return false;
		}

		if (DiscoveredNodes2.Num() == 0)
		{
			AddError(TEXT("The second transport did not discover any nodes"));
			return false;
		}

		TestTrue(TEXT("The first transport must discover exactly one node"), DiscoveredNodes1.Num() == 1);
		TestTrue(TEXT("The second transport must discover exactly one node"), DiscoveredNodes2.Num() == 1);
		TestTrue(TEXT("The discovered node IDs must be valid"), DiscoveredNodes1[0].IsValid() && DiscoveredNodes2[0].IsValid());
		TestTrue(TEXT("The discovered node IDs must be unique"), DiscoveredNodes1[0] != DiscoveredNodes2[0]);
	}

	if (HasAnyErrors())
	{
		return false;
	}

	// stress test message sending
	{
		const FDateTime StartTime = FDateTime::UtcNow();

		for (int32 Count = 0; Count < NumTestMessages; ++Count)
		{
			FUdpMockMessage* Message = FMessageEndpoint::MakeMessage<FUdpMockMessage>(MessageSize);
			// Send messages reliably otherwise they may not send due to congestion and potentially dropped by the processor.
			TSharedRef<IMessageContext, ESPMode::ThreadSafe> Context = MakeShareable(new FUdpMockMessageContext(Message, StartTime, EMessageFlags::Reliable));

			Transport1.Publish(Context);
		}

		AddInfo(FString::Printf(TEXT("Sent %i messages in %s"), NumTestMessages, *(FDateTime::UtcNow() - StartTime).ToString()));

		while ((Transport2.GetNumReceivedMessages() < NumTestMessages) && ((FDateTime::UtcNow() - StartTime) < FTimespan::FromSeconds(120.0)))
		{
			FPlatformProcess::Sleep(0.0f);
		}

		AddInfo(FString::Printf(TEXT("Received %i messages in %s"), Transport2.GetNumReceivedMessages(), *(FDateTime::UtcNow() - StartTime).ToString()));
		TestTrue(TEXT("All sent messages must have been received"), Transport2.GetNumReceivedMessages() == NumTestMessages);
	}

	if (Transport2.GetNumReceivedMessages() != NumTestMessages)
	{
		return false;
	}

	{
		const FDateTime StartTime = FDateTime::UtcNow();

		// 100 MB Test message.
		const int32 LargeMessageSize = 1024*1024*100;
		FUdpMockMessage* MockMessageToSend = FMessageEndpoint::MakeMessage<FUdpMockMessage>(LargeMessageSize);
		// Send messages reliably otherwise they may not send due to congestion and potentially dropped by the processor.
		TSharedRef<IMessageContext, ESPMode::ThreadSafe> Context = MakeShareable(new FUdpMockMessageContext(MockMessageToSend, StartTime, EMessageFlags::Reliable));

		const uint32 SenderCRC = MockMessageToSend->ComputeCRC();
		Transport1.Publish(Context);

		while ((Transport2.GetNumReceivedMessages() < NumTestMessages+1) && ((FDateTime::UtcNow() - StartTime) < FTimespan::FromSeconds(120.0)))
		{
			FPlatformProcess::Sleep(0.0f);
		}

		if ((Transport2.GetNumReceivedMessages() == NumTestMessages+1))
		{
			TSharedPtr<IMessageContext, ESPMode::ThreadSafe> ReceivedContext = Transport2.GetLastMessage();
			const UScriptStruct* MessageTypeInfo = Context->GetMessageTypeInfo().Get();

			const bool bIsAMockMessage = MessageTypeInfo->IsChildOf(FUdpMockMessage::StaticStruct());
			TestTrue(TEXT("Is Udp Mock Message"), bIsAMockMessage);

			if (bIsAMockMessage)
			{
				const FUdpMockMessage* ReceivedMockMessage = static_cast<const FUdpMockMessage*>(Context->GetMessage());
				TestTrue(TEXT("Mock message size matches"), ReceivedMockMessage->Data.Num() == MockMessageToSend->Data.Num());
				const uint32 ReceiverCRC = ReceivedMockMessage->ComputeCRC();
				TestTrue(TEXT("CRC Match for large message send. "), ReceiverCRC == SenderCRC);
			}

			AddInfo(TEXT("Completed large message test."));
		}
	}

	return true;
}


void EmptyLinkFunctionForStaticInitializationUdpMessageTransportTest()
{
	// This function exists to prevent the object file containing this test from being excluded by the linker, because it has no publicly referenced symbols.
}
