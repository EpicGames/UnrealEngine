// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"
#include "IMessageContext.h"
#include "Misc/Crc.h"
#include "Misc/DateTime.h"
#include "UdpMessagingTestTypes.generated.h"

USTRUCT()
struct FUdpMockMessage
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY(meta=(IgnoreForMemberInitializationTest))
	TArray<uint8> Data;

	FUdpMockMessage()
	{
		Data.AddUninitialized(64);
	}

	FUdpMockMessage(int32 DataSize)
	{
		Data.AddUninitialized(DataSize);
	}

	bool Serialize(FArchive& Ar)
	{
		int32 Num = Data.Num();
		Ar << Num;
		if (Ar.IsLoading())
		{
			Data.AddUninitialized(Num);
		}
		Ar.Serialize(Data.GetData(), Num);
		return true;
	}

	uint32 ComputeCRC() const
	{
		uint32 CRC = 0;
		return FCrc::MemCrc32(Data.GetData(), Data.Num(), CRC);
	}
};


class FUdpMockMessageContext
	: public IMessageContext
{
public:

	FUdpMockMessageContext(FUdpMockMessage* InMessage, const FDateTime& InTimeSent, EMessageFlags SentFlags = EMessageFlags::None)
		: Expiration(FDateTime::MaxValue())
		, Message(InMessage)
		, Scope(EMessageScope::Network)
		, Flags(SentFlags)
		, SenderThread(ENamedThreads::AnyThread)
		, TimeSent(InTimeSent)
		, TypeInfo(FUdpMockMessage::StaticStruct())
	{
		FMessageAddress::Parse(TEXT("11111111-22222222-33333333-44444444"), Sender);
	}

	~FUdpMockMessageContext()
	{
		if (Message != nullptr)
		{
			if (TypeInfo.IsValid())
			{
				TypeInfo->DestroyStruct(Message);
			}

			FMemory::Free(Message);
		}
	}

public:

	//~ IMessageContext interface

	virtual const TMap<FName, FString>& GetAnnotations() const override { return Annotations; }
	virtual TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> GetAttachment() const override { return Attachment; }
	virtual const FDateTime& GetExpiration() const override { return Expiration; }
	virtual const void* GetMessage() const override { return Message; }
	virtual const TWeakObjectPtr<UScriptStruct>& GetMessageTypeInfo() const override { return TypeInfo; }
	virtual TSharedPtr<IMessageContext, ESPMode::ThreadSafe> GetOriginalContext() const override { return OriginalContext; }
	virtual const TArray<FMessageAddress>& GetRecipients() const override { return Recipients; }
	virtual EMessageScope GetScope() const override { return Scope; }
	virtual EMessageFlags GetFlags() const override { return Flags; }
	virtual const FMessageAddress& GetSender() const override { return Sender; }
	virtual const FMessageAddress& GetForwarder() const override { return Sender; }
	virtual ENamedThreads::Type GetSenderThread() const override { return SenderThread; }
	virtual const FDateTime& GetTimeForwarded() const override { return TimeSent; }
	virtual const FDateTime& GetTimeSent() const override { return TimeSent; }

private:

	TMap<FName, FString> Annotations;
	TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> Attachment;
	FDateTime Expiration;
	void* Message;
	TSharedPtr<IMessageContext, ESPMode::ThreadSafe> OriginalContext;
	TArray<FMessageAddress> Recipients;
	EMessageScope Scope;
	EMessageFlags Flags;
	FMessageAddress Sender;
	ENamedThreads::Type SenderThread;
	FDateTime TimeSent;
	TWeakObjectPtr<UScriptStruct> TypeInfo;
};
