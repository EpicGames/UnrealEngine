// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "UdpMessagingPrivate.h"
#include "Shared/UdpMessagingSettings.h"
#include "MessageEndpoint.h"
#include "Transport/UdpSerializedMessage.h"
#include "Transport/UdpSerializeMessageTask.h"
#include "Tests/UdpMessagingTestTypes.h"


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FUdpSerializeMessageTaskTest, "System.Core.Messaging.Transports.Udp.UdpSerializedMessage", EAutomationTestFlags_ApplicationContextMask | EAutomationTestFlags::EngineFilter)


namespace UdpSerializeMessageTaskTest
{
	const FTimespan MaxWaitTime(0, 0, 10);
}


bool FUdpSerializeMessageTaskTest::RunTest(const FString& Parameters)
{
	using namespace UdpSerializeMessageTaskTest;

	auto RunTestWithMessageFormat = [&](EUdpMessageFormat MessageFormat, uint8 ProtocolVersion)
	{
		const auto TimeSent = FDateTime(2015, 9, 17, 10, 59, 23, 666);
		const auto Context = MakeShared<FUdpMockMessageContext, ESPMode::ThreadSafe>(FMessageEndpoint::MakeMessage<FUdpMockMessage>(), TimeSent);

		// synchronous reference serialization
		const auto Message1 = MakeShared<FUdpSerializedMessage, ESPMode::ThreadSafe>(MessageFormat, ProtocolVersion, Context->GetFlags());

		FUdpSerializeMessageTask Task1(Context, Message1, nullptr);
		{
			Task1.DoTask(FTaskGraphInterface::Get().GetCurrentThreadIfKnown(), FGraphEventRef());
		}

		// asynchronous serialization
		TSharedRef<FEvent, ESPMode::ThreadSafe> CompletionEvent = MakeShareable(FPlatformProcess::GetSynchEventFromPool(), [](FEvent* EventToDelete)
		{
			FPlatformProcess::ReturnSynchEventToPool(EventToDelete);
		});

		TSharedRef<FUdpSerializedMessage, ESPMode::ThreadSafe> Message2 = MakeShared<FUdpSerializedMessage, ESPMode::ThreadSafe>(MessageFormat, ProtocolVersion, Context->GetFlags());
		TGraphTask<FUdpSerializeMessageTask>::CreateTask().ConstructAndDispatchWhenReady(Context, Message2, CompletionEvent);

		const bool Completed = CompletionEvent->Wait(MaxWaitTime);

		const TArray<uint8>& DataArray1 = Message1->GetDataArray();
		const TArray<uint8>& DataArray2 = Message2->GetDataArray();

		const void* Data1 = DataArray1.GetData();
		const void* Data2 = DataArray2.GetData();

		const bool Equal = (FMemory::Memcmp(Data1, Data2, DataArray1.Num()) == 0);

		TestEqual(TEXT("Synchronous message serialization must succeed"), Message1->GetState(), EUdpSerializedMessageState::Complete);
		TestTrue(TEXT("Asynchronous message serialization must complete"), Completed);
		TestEqual(TEXT("Asynchronous message serialization must succeed"), Message2->GetState(), EUdpSerializedMessageState::Complete);
		TestTrue(TEXT("Synchronous and asynchronous message serialization must yield same results"), Equal);

		return (Completed && Equal);
	};

	bool JsonTest = RunTestWithMessageFormat(EUdpMessageFormat::Json, 10); // Json was supported by UDP message protocol up to version 10.
	bool PlatformEndianTest = RunTestWithMessageFormat(EUdpMessageFormat::CborPlatformEndianness, UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION); // CBOR using platform endianness was added at version 11.
	bool StandardEndianTest = RunTestWithMessageFormat(EUdpMessageFormat::CborStandardEndianness, UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION); // CBOR using standard endianness was added at version 14.

	return JsonTest && PlatformEndianTest && StandardEndianTest;
}

void EmptyLinkFunctionForStaticInitializationUdpSerializeMessageTaskTest()
{
	// This function exists to prevent the object file containing this test from being excluded by the linker, because it has no publically referenced symbols.
}
