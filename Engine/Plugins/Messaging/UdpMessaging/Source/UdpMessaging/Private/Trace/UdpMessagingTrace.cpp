// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/UdpMessagingTrace.h"


namespace UE::UdpMessaging::Trace
{
	const TCHAR* LexToString(EUdpMessageLifecycleEvent InEvent)
	{
		switch (InEvent)
		{
		case EUdpMessageLifecycleEvent::EnqueueOutbound: return TEXT("EnqueueOutbound");
		case EUdpMessageLifecycleEvent::BeginSerialize: return TEXT("BeginSerialize");
		case EUdpMessageLifecycleEvent::EndSerialize: return TEXT("EndSerialize");
		case EUdpMessageLifecycleEvent::ConsumeOutbound: return TEXT("ConsumeOutbound");
		case EUdpMessageLifecycleEvent::Complete: return TEXT("Complete");

		case EUdpMessageLifecycleEvent::FirstSegmentEnqueueInbound: return TEXT("FirstSegmentEnqueueInbound");
		case EUdpMessageLifecycleEvent::BeginReassembly: return TEXT("BeginReassembly");
		case EUdpMessageLifecycleEvent::EndReassembly: return TEXT("EndReassembly");
		case EUdpMessageLifecycleEvent::PendingDelivery: return TEXT("PendingDelivery");
		case EUdpMessageLifecycleEvent::BeginDeserialize: return TEXT("BeginDeserialize");
		case EUdpMessageLifecycleEvent::Delivered: return TEXT("Delivered");

		case EUdpMessageLifecycleEvent::Discarded: return TEXT("Discarded");

		default:
			return TEXT("Unknown");
		}
	}
}


#if UDPMESSAGINGTRACE_ENABLED

#include "HAL/PlatformTime.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/Guid.h"
#include "Misc/TVariant.h"
#include "ProfilingDebugging/StringsTrace.h"
#include "Trace/Trace.inl"
#include "UObject/TopLevelAssetPath.h"

#include <type_traits>


UE_TRACE_EVENT_BEGIN(UdpMessaging, DiscoveredNode, Important|NoSync)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint16, ShortId) // Lightweight primary key, auto-increment, assumes we see < 64k unique nodes over life of app
	UE_TRACE_EVENT_FIELD(uint16, DiscoveredByShortId) // Our local short ID which discovered this one; if 0, the "discovered" node is a new local processor
	UE_TRACE_EVENT_FIELD(uint32, IdA)
	UE_TRACE_EVENT_FIELD(uint32, IdB)
	UE_TRACE_EVENT_FIELD(uint32, IdC)
	UE_TRACE_EVENT_FIELD(uint32, IdD)
	UE_TRACE_EVENT_FIELD(uint8,  IpAddrA)
	UE_TRACE_EVENT_FIELD(uint8,  IpAddrB)
	UE_TRACE_EVENT_FIELD(uint8,  IpAddrC)
	UE_TRACE_EVENT_FIELD(uint8,  IpAddrD)
	UE_TRACE_EVENT_FIELD(uint16, Port)
	UE_TRACE_EVENT_FIELD(uint8,  ProtocolVersion)
UE_TRACE_EVENT_END()


UE_TRACE_EVENT_BEGIN(UdpMessaging, MessageSummary, NoSync)
	UE_TRACE_EVENT_FIELD(uint16, SenderShortId)
	UE_TRACE_EVENT_FIELD(uint16, RecipientShortId)
	UE_TRACE_EVENT_FIELD(uint32, MessageId)
	UE_TRACE_EVENT_FIELD(uint64, Size)
	UE_TRACE_EVENT_FIELD(uint32, NumSegments)
	UE_TRACE_EVENT_FIELD(uint32, Flags)
UE_TRACE_EVENT_END()


UE_TRACE_EVENT_BEGIN(UdpMessaging, MessageTypeInfo, NoSync)
	UE_TRACE_EVENT_FIELD(uint16, SenderShortId)
	UE_TRACE_EVENT_FIELD(uint16, RecipientShortId)
	UE_TRACE_EVENT_FIELD(uint32, MessageId)
	UE_TRACE_EVENT_REFERENCE_FIELD(Strings, FName, Package) // FTopLevelAssetPath.PackageName or None
	UE_TRACE_EVENT_REFERENCE_FIELD(Strings, FName, Asset) // FTopLevelAssetPath.AssetName or combined
UE_TRACE_EVENT_END()

UE_TRACE_EVENT_BEGIN(UdpMessaging, MessageLifecycle)
	UE_TRACE_EVENT_FIELD(uint64, Cycle)
	UE_TRACE_EVENT_FIELD(uint16, SenderShortId)
	UE_TRACE_EVENT_FIELD(uint16, RecipientShortId)
	UE_TRACE_EVENT_FIELD(uint32, MessageId)
	UE_TRACE_EVENT_FIELD(uint8,  LifecycleEvent)
UE_TRACE_EVENT_END()


namespace UE::UdpMessaging::Trace
{
	void OutputDiscoveredNode(uint16 ShortId, uint16 DiscoveredByShortId, const FGuid& NodeId, const FIPv4Endpoint& Endpoint, uint8 ProtocolVersion)
	{
		UE_TRACE_LOG(UdpMessaging, DiscoveredNode, MessagingChannel)
			<< DiscoveredNode.Cycle(FPlatformTime::Cycles64())
			<< DiscoveredNode.ShortId(ShortId)
			<< DiscoveredNode.DiscoveredByShortId(DiscoveredByShortId)
			<< DiscoveredNode.IdA(NodeId.A)
			<< DiscoveredNode.IdB(NodeId.B)
			<< DiscoveredNode.IdC(NodeId.C)
			<< DiscoveredNode.IdD(NodeId.D)
			<< DiscoveredNode.IpAddrA(Endpoint.Address.A)
			<< DiscoveredNode.IpAddrB(Endpoint.Address.B)
			<< DiscoveredNode.IpAddrC(Endpoint.Address.C)
			<< DiscoveredNode.IpAddrD(Endpoint.Address.D)
			<< DiscoveredNode.Port(Endpoint.Port)
			<< DiscoveredNode.ProtocolVersion(ProtocolVersion);
	}

	void OutputMessageSummary(uint16 SenderShortId, uint16 RecipientShortId, uint32 MessageId, uint64 Size, uint32 NumSegments, EMessageFlags Flags)
	{
		UE_TRACE_LOG(UdpMessaging, MessageSummary, MessagingChannel)
			<< MessageSummary.SenderShortId(SenderShortId)
			<< MessageSummary.RecipientShortId(RecipientShortId)
			<< MessageSummary.MessageId(MessageId)
			<< MessageSummary.Size(Size)
			<< MessageSummary.NumSegments(NumSegments)
			<< MessageSummary.Flags(static_cast<std::underlying_type_t<EMessageFlags>>(Flags));
	}

	void OutputMessageTypeInfo(uint16 SenderShortId, uint16 RecipientShortId, uint32 MessageId, const TVariant<FName, FTopLevelAssetPath>& TypeInfo)
	{
		const FTopLevelAssetPath* Path = TypeInfo.TryGet<FTopLevelAssetPath>();
		const FName Package = Path ? Path->GetPackageName() : NAME_None;
		const FName Asset = Path ? Path->GetAssetName() : TypeInfo.Get<FName>();

		const UE::Trace::FEventRef32 PackageRef = FStringTrace::GetNameRef(Package);
		const UE::Trace::FEventRef32 AssetRef = FStringTrace::GetNameRef(Asset);

		UE_TRACE_LOG(UdpMessaging, MessageTypeInfo, MessagingChannel)
			<< MessageTypeInfo.SenderShortId(SenderShortId)
			<< MessageTypeInfo.RecipientShortId(RecipientShortId)
			<< MessageTypeInfo.MessageId(MessageId)
			<< MessageTypeInfo.Package(PackageRef)
			<< MessageTypeInfo.Asset(AssetRef);
	}

	void OutputMessageLifecycle(uint64 Cycle, uint16 SenderShortId, uint16 RecipientShortId, uint32 MessageId, EUdpMessageLifecycleEvent Event)
	{
		UE_TRACE_LOG(UdpMessaging, MessageLifecycle, MessagingChannel)
			<< MessageLifecycle.Cycle(Cycle)
			<< MessageLifecycle.SenderShortId(SenderShortId)
			<< MessageLifecycle.RecipientShortId(RecipientShortId)
			<< MessageLifecycle.MessageId(MessageId)
			<< MessageLifecycle.LifecycleEvent(static_cast<std::underlying_type_t<EUdpMessageLifecycleEvent>>(Event));
	}
};

#endif // #if UDPMESSAGINGTRACE_ENABLED
