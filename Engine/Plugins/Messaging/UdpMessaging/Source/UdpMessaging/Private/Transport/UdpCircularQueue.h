// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/CircularBuffer.h"

/**
 * Implements a first-in first-out queue using a circular array.
 *
 * This class is not thread safe and should be used  only in single-producer single-consumer scenarios.
 *
 * The number of items that can be enqueued is one less than the queue's capacity,
 * because one item will be used for detecting full and empty states.
 *
 * @param ElementType The type of elements held in the queue.
 */
template<typename ElementType> class TUdpCircularQueue
{
public:

	/**
	 * Constructor.
	 *
	 * @param CapacityPlusOne The number of elements that the queue can hold (will be rounded up to the next power of 2).
	 */
	explicit TUdpCircularQueue(uint32 CapacityPlusOne)
		: Buffer(CapacityPlusOne)
		, Head(0)
		, Tail(0)
	{ }

public:

	/**
	 * Gets the number of elements in the queue.
	 *
	 * Can be called from any thread. The result reflects the calling thread's current
	 * view. Since no locking is used, different threads may return different results.
	 *
	 * @return Number of queued elements.
	 */
	uint32 Count() const
	{
		int32 Count = Tail - Head;

		if (Count < 0)
		{
			Count += Buffer.Capacity();
		}

		return (uint32)Count;
	}

	/**
	 * Removes an item from the front of the queue.
	 *
	 * @param OutElement Will contain the element if the queue is not empty.
	 * @return true if an element has been returned, false if the queue was empty.
	 */
	bool Dequeue(ElementType& OutElement)
	{
		const uint32 CurrentHead = Head;

		if (CurrentHead != Tail)
		{
			OutElement = MoveTemp(Buffer[CurrentHead]);
			Head  = Buffer.GetNextIndex(CurrentHead);

			return true;
		}

		return false;
	}

	/**
	 * Removes an item from the front of the queue.
	 *
	 * @return true if an element has been removed, false if the queue was empty.
	 */
	bool Dequeue()
	{
		const uint32 CurrentHead = Head;

		if (CurrentHead != Tail)
		{
			Head = Buffer.GetNextIndex(CurrentHead);

			return true;
		}

		return false;
	}

	/**
	 * Empties the queue.
	 *
	 * @see IsEmpty
	 */
	void Empty()
	{
		Head = Tail;
	}

	/**
	 * Adds an item to the end of the queue.
	 *
	 * @param Element The element to add.
	 * @return true if the item was added, false if the queue was full.
	 */
	bool Enqueue(const ElementType& Element)
	{
		const uint32 CurrentTail = Tail;
		uint32 NewTail = Buffer.GetNextIndex(CurrentTail);

		if (NewTail != Head)
		{
			Buffer[CurrentTail] = Element;
			Tail = NewTail;

			return true;
		}

		return false;
	}

	/**
	 * Adds an item to the end of the queue.
	 *
	 * @param Element The element to add.
	 * @return true if the item was added, false if the queue was full.
	 */
	bool Enqueue(ElementType&& Element)
	{
		const uint32 CurrentTail = Tail;
		uint32 NewTail = Buffer.GetNextIndex(CurrentTail);

		if (NewTail != Head)
		{
			Buffer[CurrentTail] = MoveTemp(Element);
			Tail = NewTail;

			return true;
		}

		return false;
	}

	/**
	 * Checks whether the queue is empty.
	 *
	 * Can be called from any thread. The result reflects the calling thread's current
	 * view. Since no locking is used, different threads may return different results.
	 *
	 * @return true if the queue is empty, false otherwise.
	 * @see Empty, IsFull
	 */
	FORCEINLINE bool IsEmpty() const
	{
		return (Head == Tail);
	}

	/**
	 * Checks whether the queue is full.
	 *
	 * Can be called from any thread. The result reflects the calling thread's current
	 * view. Since no locking is used, different threads may return different results.
	 *
	 * @return true if the queue is full, false otherwise.
	 * @see IsEmpty
	 */
	bool IsFull() const
	{
		return (Buffer.GetNextIndex(Tail) == Head);
	}

	/**
	 * Returns the oldest item in the queue without removing it.
	 *
	 * @param OutItem Will contain the item if the queue is not empty.
	 * @return true if an item has been returned, false if the queue was empty.
	 */
	bool Peek(ElementType& OutItem) const
	{
		const uint32 CurrentHead = Head;

		if (CurrentHead != Tail)
		{
			OutItem = Buffer[CurrentHead];

			return true;
		}

		return false;
	}

	/**
	 * Returns the oldest item in the queue without removing it.
	 *
	 * @return an ElementType pointer if an item has been returned, nullptr if the queue was empty.
	 * @note The return value is only valid until Dequeue, Empty, or the destructor has been called.
	 */
	const ElementType* Peek() const
	{
		const uint32 CurrentHead = Head;

		if (CurrentHead != Tail)
		{
			return &Buffer[CurrentHead];
		}

		return nullptr;
	}

private:

	/** Holds the buffer. */
	TCircularBuffer<ElementType> Buffer;

	/** Holds the index to the first item in the buffer. */
	uint32 Head;

	/** Holds the index to the last item in the buffer. */
	uint32 Tail;
};
