// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transport/UdpDeserializedMessage.h"

#include "Async/TaskGraphInterfaces.h"
#include "Backends/JsonStructDeserializerBackend.h"
#include "Backends/CborStructDeserializerBackend.h"
#include "Misc/AssertionMacros.h"
#include "StructDeserializer.h"
#include "Serialization/MemoryReader.h"
#include "Transport/UdpReassembledMessage.h"
#include "Shared/UdpMessagingSettings.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Trace/UdpMessagingTrace.h"

/* FUdpDeserializedMessage structors
*****************************************************************************/

FUdpDeserializedMessage::FUdpDeserializedMessage(const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& InAttachment)
	: Attachment(InAttachment)
	, MessageData(nullptr)
	, Flags(EMessageFlags::None)
{ }


FUdpDeserializedMessage::~FUdpDeserializedMessage()
{
	if (MessageData != nullptr)
	{
		if (TypeInfo.IsValid())
		{
			TypeInfo->DestroyStruct(MessageData);
		}

		FMemory::Free(MessageData);
		MessageData = nullptr;
	}
}

/** Helper class for protocol deserialization dispatching */
class FUdpDeserializedMessageDetails
{
public:
	static bool DeserializeV10(FUdpDeserializedMessage& DeserializedMessage, FMemoryReader& MessageReader);
	static bool DeserializeV11_15(FUdpDeserializedMessage& DeserializedMessage, FMemoryReader& MessageReader, bool bIsLWCBackwardCompatibilityMode);
	static bool DeserializeV16_18(FUdpDeserializedMessage& DeserializedMessage, FMemoryReader& MessageReader);
	static bool Deserialize(FUdpDeserializedMessage& DeserializedMessage, const FUdpReassembledMessage& ReassembledMessage);

	template <typename ReturnType>
	static ReturnType GetMessageType(FMemoryReader& MessageReader);
};

/* FUdpDeserializedMessage interface
 *****************************************************************************/

bool FUdpDeserializedMessage::Deserialize(const FUdpReassembledMessage& ReassembledMessage)
{
	SCOPED_MESSAGING_TRACE(FUdpDeserializedMessage_Deserialize);
	return FUdpDeserializedMessageDetails::Deserialize(*this, ReassembledMessage);
}

/* IMessageContext interface
 *****************************************************************************/

const TMap<FName, FString>& FUdpDeserializedMessage::GetAnnotations() const
{
	return Annotations;
}


TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> FUdpDeserializedMessage::GetAttachment() const
{
	return Attachment;
}


const FDateTime& FUdpDeserializedMessage::GetExpiration() const
{
	return Expiration;
}


const void* FUdpDeserializedMessage::GetMessage() const
{
	return MessageData;
}


const TWeakObjectPtr<UScriptStruct>& FUdpDeserializedMessage::GetMessageTypeInfo() const
{
	return TypeInfo;
}


TSharedPtr<IMessageContext, ESPMode::ThreadSafe> FUdpDeserializedMessage::GetOriginalContext() const
{
	return nullptr;
}


const TArray<FMessageAddress>& FUdpDeserializedMessage::GetRecipients() const
{
	return Recipients;
}


EMessageScope FUdpDeserializedMessage::GetScope() const
{
	return Scope;
}


EMessageFlags FUdpDeserializedMessage::GetFlags() const
{
	return Flags;
}


const FMessageAddress& FUdpDeserializedMessage::GetSender() const
{
	return Sender;
}


const FMessageAddress& FUdpDeserializedMessage::GetForwarder() const
{
	return Sender;
}


ENamedThreads::Type FUdpDeserializedMessage::GetSenderThread() const
{
	return ENamedThreads::AnyThread;
}


const FDateTime& FUdpDeserializedMessage::GetTimeForwarded() const
{
	return TimeSent;
}


const FDateTime& FUdpDeserializedMessage::GetTimeSent() const
{
	return TimeSent;
}


/* FUdpDeserializedMessageDetails implementation
*****************************************************************************/

TWeakObjectPtr<UScriptStruct> DeserializeSafeLookupForFindObject_InThread(FStringView TypeName)
{
	UUdpMessagingSettings const* CurrentSettings = GetDefault<UUdpMessagingSettings>();
	const bool bRunningInTaskThread = (CurrentSettings->MaxConcurrentDeserializationTasks > 1);
	if (bRunningInTaskThread)
	{
		FGCScopeGuard Guard;
		return FindFirstObjectSafe<UScriptStruct>(TypeName, EFindFirstObjectOptions::EnsureIfAmbiguous);
	}
	else
	{
		// If we're not in a task, we want to avoid potentially stalling the processor thread.
		FGCScopeTryGuard MaybeGuard;
		if (MaybeGuard.LockSucceeded())
		{
			return FindFirstObjectSafe<UScriptStruct>(TypeName, EFindFirstObjectOptions::EnsureIfAmbiguous);
		}
	}

	return nullptr;
}

template <typename ReturnType>
ReturnType FUdpDeserializedMessageDetails::GetMessageType(FMemoryReader& MessageReader)
{
	// This should only be used for TopLevelAssetPath or FName paths
	static_assert(std::is_same<FName, ReturnType>::value
				  || std::is_same<FTopLevelAssetPath, ReturnType>::value);

	ReturnType MessageType;
	MessageReader << MessageType;
	return MessageType;
}

bool FUdpDeserializedMessageDetails::DeserializeV10(FUdpDeserializedMessage& DeserializedMessage, FMemoryReader& MessageReader)
{
	// message type info
	{
		FName MessageType = GetMessageType<FName>(MessageReader);

		if (DeserializedMessage.TypeInfo == nullptr)
		{
			DeserializedMessage.TypeInfo = DeserializeSafeLookupForFindObject_InThread(*MessageType.ToString());
		}

		if (!DeserializedMessage.TypeInfo.IsValid(false, true))
		{
			return false;
		}
	}

	// sender address
	{
		MessageReader << DeserializedMessage.Sender;
	}

	// recipient addresses
	{
		int32 NumRecipients = 0;
		MessageReader << NumRecipients;

		if ((NumRecipients < 0) || (NumRecipients > UDP_MESSAGING_MAX_RECIPIENTS))
		{
			return false;
		}

		DeserializedMessage.Recipients.Empty(NumRecipients);

		while (0 < NumRecipients--)
		{
			MessageReader << DeserializedMessage.Recipients.AddDefaulted_GetRef();
		}
	}

	// message scope
	{
		MessageReader << DeserializedMessage.Scope;

		if (DeserializedMessage.Scope > EMessageScope::All)
		{
			return false;
		}
	}

	// time sent & expiration
	{
		MessageReader << DeserializedMessage.TimeSent;
		MessageReader << DeserializedMessage.Expiration;
	}

	// annotations
	{
		int32 NumAnnotations = 0;
		MessageReader << NumAnnotations;

		if (NumAnnotations > UDP_MESSAGING_MAX_ANNOTATIONS)
		{
			return false;
		}

		while (0 < NumAnnotations--)
		{
			FName Key;
			FString Value;

			MessageReader << Key;
			MessageReader << Value;

			DeserializedMessage.Annotations.Add(Key, Value);
		}
	}

	// create message body
	DeserializedMessage.MessageData = FMemory::Malloc(DeserializedMessage.TypeInfo->GetStructureSize());
	DeserializedMessage.TypeInfo->InitializeStruct(DeserializedMessage.MessageData);

	// deserialize message body
	FJsonStructDeserializerBackend Backend(MessageReader);
	return FStructDeserializer::Deserialize(DeserializedMessage.MessageData, *DeserializedMessage.TypeInfo, Backend);
}

bool FUdpDeserializedMessageDetails::DeserializeV11_15(FUdpDeserializedMessage& DeserializedMessage, FMemoryReader& MessageReader, bool bIsLWCBackwardCompatibilityMode)
{
	// message type info
	{
		FName MessageType = GetMessageType<FName>(MessageReader);

		if (DeserializedMessage.TypeInfo == nullptr)
		{
			DeserializedMessage.TypeInfo = DeserializeSafeLookupForFindObject_InThread(*MessageType.ToString());
		}

		if (!DeserializedMessage.TypeInfo.IsValid(false, true))
		{
			UE_LOGF(LogUdpMessaging, Verbose, "No valid type info found for message type %ls", *MessageType.ToString());
			return false;
		}
	}

	// sender address
	{
		MessageReader << DeserializedMessage.Sender;
	}

	// recipient addresses
	{
		int32 NumRecipients = 0;
		MessageReader << NumRecipients;

		if ((NumRecipients < 0) || (NumRecipients > UDP_MESSAGING_MAX_RECIPIENTS))
		{
			return false;
		}

		DeserializedMessage.Recipients.Empty(NumRecipients);

		while (0 < NumRecipients--)
		{
			MessageReader << DeserializedMessage.Recipients.AddDefaulted_GetRef();
		}
	}

	// message scope
	{
		MessageReader << DeserializedMessage.Scope;

		if (DeserializedMessage.Scope > EMessageScope::All)
		{
			return false;
		}
	}

	// message flags
	{
		MessageReader << DeserializedMessage.Flags;
	}

	// time sent & expiration
	{
		MessageReader << DeserializedMessage.TimeSent;
		MessageReader << DeserializedMessage.Expiration;
	}

	// annotations
	{
		int32 NumAnnotations = 0;
		MessageReader << NumAnnotations;

		if (NumAnnotations > UDP_MESSAGING_MAX_ANNOTATIONS)
		{
			return false;
		}

		while (0 < NumAnnotations--)
		{
			FName Key;
			FString Value;

			MessageReader << Key;
			MessageReader << Value;

			DeserializedMessage.Annotations.Add(Key, Value);
		}
	}

	// wire format 
	uint8 FormatId;
	MessageReader << FormatId;
	EUdpMessageFormat MessageFormat = (EUdpMessageFormat)FormatId;
	
	// create message body
	DeserializedMessage.MessageData = FMemory::Malloc(DeserializedMessage.TypeInfo->GetStructureSize());
	DeserializedMessage.TypeInfo->InitializeStruct(DeserializedMessage.MessageData);

	switch (MessageFormat)
	{
	case EUdpMessageFormat::Json:
	{
		// deserialize json
		FJsonStructDeserializerBackend Backend(MessageReader);
		return FStructDeserializer::Deserialize(DeserializedMessage.MessageData, *DeserializedMessage.TypeInfo, Backend);
	}
	case EUdpMessageFormat::CborPlatformEndianness:
	{
		// deserialize cbor (using this platform endianness).
		FCborStructDeserializerBackend Backend(MessageReader, ECborEndianness::Platform, bIsLWCBackwardCompatibilityMode);
		return FStructDeserializer::Deserialize(DeserializedMessage.MessageData, *DeserializedMessage.TypeInfo, Backend);
	}
	case EUdpMessageFormat::CborStandardEndianness:
	{
		// deserialize cbor (using the CBOR standard endianness - big endian).
		FCborStructDeserializerBackend Backend(MessageReader, ECborEndianness::StandardCompliant, bIsLWCBackwardCompatibilityMode);
		return FStructDeserializer::Deserialize(DeserializedMessage.MessageData, *DeserializedMessage.TypeInfo, Backend);
	}
	case EUdpMessageFormat::TaggedProperty:
	{
		// deserialize message body using tagged property
		// Hack : this binary serialization should use a more standard protocol, should use cbor
		DeserializedMessage.TypeInfo->SerializeItem(MessageReader, DeserializedMessage.MessageData, nullptr);
		return !MessageReader.GetError();
	}
	default:
		// Unsupported format
		return false;
	}
}

bool FUdpDeserializedMessageDetails::DeserializeV16_18(FUdpDeserializedMessage& DeserializedMessage, FMemoryReader& MessageReader)
{
	// message type info
	{
		FTopLevelAssetPath MessageType = GetMessageType<FTopLevelAssetPath>(MessageReader);

		if (DeserializedMessage.TypeInfo == nullptr)
		{
			DeserializedMessage.TypeInfo = DeserializeSafeLookupForFindObject_InThread(*MessageType.ToString());
		}

		if (!DeserializedMessage.TypeInfo.IsValid(false, true))
		{
			UE_LOGF(LogUdpMessaging, Verbose, "No valid type info found for message type %ls", *MessageType.ToString());
			return false;
		}
	}

	// sender address
	{
		MessageReader << DeserializedMessage.Sender;
	}

	// recipient addresses
	{
		int32 NumRecipients = 0;
		MessageReader << NumRecipients;

		if ((NumRecipients < 0) || (NumRecipients > UDP_MESSAGING_MAX_RECIPIENTS))
		{
			return false;
		}

		DeserializedMessage.Recipients.Empty(NumRecipients);

		while (0 < NumRecipients--)
		{
			MessageReader << DeserializedMessage.Recipients.AddDefaulted_GetRef();
		}
	}

	// message scope
	{
		MessageReader << DeserializedMessage.Scope;

		if (DeserializedMessage.Scope > EMessageScope::All)
		{
			return false;
		}
	}

	// message flags
	{
		MessageReader << DeserializedMessage.Flags;
	}

	// time sent & expiration
	{
		MessageReader << DeserializedMessage.TimeSent;
		MessageReader << DeserializedMessage.Expiration;
	}

	// annotations
	{
		int32 NumAnnotations = 0;
		MessageReader << NumAnnotations;

		if (NumAnnotations > UDP_MESSAGING_MAX_ANNOTATIONS)
		{
			return false;
		}

		while (0 < NumAnnotations--)
		{
			FName Key;
			FString Value;

			MessageReader << Key;
			MessageReader << Value;

			DeserializedMessage.Annotations.Add(Key, Value);
		}
	}

	// wire format 
	uint8 FormatId;
	MessageReader << FormatId;
	EUdpMessageFormat MessageFormat = (EUdpMessageFormat)FormatId;
	
	// create message body
	DeserializedMessage.MessageData = FMemory::Malloc(DeserializedMessage.TypeInfo->GetStructureSize());
	DeserializedMessage.TypeInfo->InitializeStruct(DeserializedMessage.MessageData);

	constexpr bool bIsLWCBackwardCompatibilityMode = false;
	switch (MessageFormat)
	{
	case EUdpMessageFormat::Json:
	{
		// deserialize json
		FJsonStructDeserializerBackend Backend(MessageReader);
		return FStructDeserializer::Deserialize(DeserializedMessage.MessageData, *DeserializedMessage.TypeInfo, Backend);
	}
	case EUdpMessageFormat::CborPlatformEndianness:
	{
		// deserialize cbor (using this platform endianness).
		FCborStructDeserializerBackend Backend(MessageReader, ECborEndianness::Platform, bIsLWCBackwardCompatibilityMode);
		return FStructDeserializer::Deserialize(DeserializedMessage.MessageData, *DeserializedMessage.TypeInfo, Backend);
	}
	case EUdpMessageFormat::CborStandardEndianness:
	{
		// deserialize cbor (using the CBOR standard endianness - big endian).
		FCborStructDeserializerBackend Backend(MessageReader, ECborEndianness::StandardCompliant, bIsLWCBackwardCompatibilityMode);
		return FStructDeserializer::Deserialize(DeserializedMessage.MessageData, *DeserializedMessage.TypeInfo, Backend);
	}
	case EUdpMessageFormat::TaggedProperty:
	{
		// deserialize message body using tagged property
		// Hack : this binary serialization should use a more standard protocol, should use cbor
		DeserializedMessage.TypeInfo->SerializeItem(MessageReader, DeserializedMessage.MessageData, nullptr);
		return !MessageReader.GetError();
	}
	default:
		// Unsupported format
		return false;
	}
}

FNameOrFTopLevel FUdpDeserializedMessage::PeekMessageTypeInfoName(const FUdpReassembledMessage& PartialMessage)
{
	check(PartialMessage.HasFirstSegment());
	FNameOrFTopLevel Path;

	FMemoryReader MessageReader(PartialMessage.GetData());
	switch (PartialMessage.GetProtocolVersion())
	{
	case 10:
	case 11:
	case 12:
	case 13:
	case 14:
	case 15:
	{
		FName MessageName = FUdpDeserializedMessageDetails::GetMessageType<FName>(MessageReader);
		Path.Set<FName>(MessageName);
		break;
	}
	case 16:
		// fallthrough
	case 17:
		// fallthrough
	case 18:
	{
		FTopLevelAssetPath AssetPath = FUdpDeserializedMessageDetails::GetMessageType<FTopLevelAssetPath>(MessageReader);
		Path.Set<FTopLevelAssetPath>(AssetPath);
		break;
	}
	default:
		UE_LOGF(LogUdpMessaging, Error, "Unsupported Protocol Version message tasked for MessageType query.");
		break;
	};
	return Path;
}

TWeakObjectPtr<UScriptStruct> FUdpDeserializedMessage::ResolvePath(const FNameOrFTopLevel& Path)
{
	TWeakObjectPtr<UScriptStruct> TypeInfo;
	if (Path.IsType<FName>())
	{
		TypeInfo = FindFirstObjectSafe<UScriptStruct>(*Path.Get<FName>().ToString(), EFindFirstObjectOptions::EnsureIfAmbiguous);
	}
	else if (Path.IsType<FTopLevelAssetPath>())
	{
		TypeInfo = FindObjectSafe<UScriptStruct>(Path.Get<FTopLevelAssetPath>());
	}
	else
	{
		checkNoEntry();
	}
	return TypeInfo;
}

bool FUdpDeserializedMessageDetails::Deserialize(FUdpDeserializedMessage& DeserializedMessage, const FUdpReassembledMessage& ReassembledMessage)
{
	// Note that some complex values are deserialized manually here, so that we
	// can sanity check their values. @see FUdpSerializeMessageTask::DoTask()

	FMemoryReader MessageReader(ReassembledMessage.GetData());
	MessageReader.ArMaxSerializeSize = NAME_SIZE;

	// We may be able to skip lookup of the type name based on precached data provided by
	// the reassembler.
	//
	DeserializedMessage.TypeInfo = ReassembledMessage.GetMessageTypeInfo();
	switch (ReassembledMessage.GetProtocolVersion())
	{
	case 10:
		return DeserializeV10(DeserializedMessage, MessageReader);

	case 11:
		// fallthrough
	case 12:
		// fallthrough
	case 13:
		// fallthrough
	case 14:
	{
		// Protocol version 11-14 requires LWC backward compat mode
		constexpr bool bIsLWCBackwardCompatibilityMode = true;
		return DeserializeV11_15(DeserializedMessage, MessageReader, bIsLWCBackwardCompatibilityMode);
	}
	case 15:
	{
		constexpr bool bIsLWCBackwardCompatibilityMode = false;
		return DeserializeV11_15(DeserializedMessage, MessageReader, bIsLWCBackwardCompatibilityMode);
	}
	case 16:
		// fallthrough
	case 17:
		// fallthrough
	case 18:
	{
		return DeserializeV16_18(DeserializedMessage, MessageReader);
	}

	default:
		UE_LOGF(LogUdpMessaging, Error, "Unsupported Protocol Version message tasked for deserialization, discarding...");
		break;
	}
	return false;
}
