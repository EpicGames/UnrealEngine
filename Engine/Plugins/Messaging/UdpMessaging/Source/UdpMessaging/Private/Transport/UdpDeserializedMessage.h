// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"
#include "Misc/DateTime.h"
#include "UObject/TopLevelAssetPath.h"
#include "Misc/TVariant.h"

class FUdpReassembledMessage;
class IMessageAttachment;
class UScriptStruct;
class FUdpDeserializedMessageDetails;

using FNameOrFTopLevel = TVariant<FName, FTopLevelAssetPath>;
/**
 * Holds a deserialized message.
 */
class FUdpDeserializedMessage
	: public IMessageContext
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InAttachment An optional message attachment.
	 */
	FUdpDeserializedMessage(const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& InAttachment);

	/** Virtual destructor. */
	virtual ~FUdpDeserializedMessage() override;

public:

	/**
	 * Deserializes the given reassembled message.
	 *
	 * @param ReassembledMessage The reassembled message to deserialize.
	 * @return true on success, false otherwise.
	 */
	bool Deserialize(const FUdpReassembledMessage& ReassembledMessage);

	/**
	 * Called early by the message processor to peek at the typeinfo name.
	 */
	static FNameOrFTopLevel PeekMessageTypeInfoName(const FUdpReassembledMessage& ReassembledMessage);
	static TWeakObjectPtr<UScriptStruct> ResolvePath(const FNameOrFTopLevel& Path);

public:

	//~ IMessageContext interface

	virtual const TMap<FName, FString>& GetAnnotations() const override;
	virtual TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> GetAttachment() const override;
	virtual const FDateTime& GetExpiration() const override;
	virtual const void* GetMessage() const override;
	virtual const TWeakObjectPtr<UScriptStruct>& GetMessageTypeInfo() const override;
	virtual TSharedPtr<IMessageContext, ESPMode::ThreadSafe> GetOriginalContext() const override;
	virtual const TArray<FMessageAddress>& GetRecipients() const override;
	virtual EMessageScope GetScope() const override;
	virtual EMessageFlags GetFlags() const override;

	virtual const FMessageAddress& GetSender() const override;
	virtual const FMessageAddress& GetForwarder() const override;
	virtual ENamedThreads::Type GetSenderThread() const override;
	virtual const FDateTime& GetTimeForwarded() const override;
	virtual const FDateTime& GetTimeSent() const override;

private:
	friend class FUdpDeserializedMessageDetails;

	/** Holds the optional message annotations. */
	TMap<FName, FString> Annotations;

	/** Holds a pointer to attached binary data. */
	TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> Attachment;

	/** Holds the expiration time. */
	FDateTime Expiration;

	/** Holds the message. */
	void* MessageData;

	/** Holds the message recipients. */
	TArray<FMessageAddress> Recipients;

	/** Holds the message's scope. */
	EMessageScope Scope;

	/** Holds the message's flags. */
	EMessageFlags Flags;

	/** Holds the sender's identifier. */
	FMessageAddress Sender;

	/** Holds the time at which the message was sent. */
	FDateTime TimeSent;

	/** Holds the message's type information. */
	TWeakObjectPtr<UScriptStruct> TypeInfo;
};
