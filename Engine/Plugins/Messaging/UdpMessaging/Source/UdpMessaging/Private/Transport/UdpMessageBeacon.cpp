// Copyright Epic Games, Inc. All Rights Reserved.


#include "Transport/UdpMessageBeacon.h"
#include "HAL/PlatformProcess.h"
#include "UdpMessagingPrivate.h"

#include "HAL/Event.h"
#include "HAL/RunnableThread.h"
#include "Serialization/ArrayWriter.h"
#include "Sockets.h"
#include "Shared/SocketSender.h"
#include "Shared/UdpMessageSegment.h"

/* FUdpMessageHelloSender static initialization
 *****************************************************************************/

const FTimespan FUdpMessageBeacon::IntervalPerEndpoint = FTimespan::FromMilliseconds(200);
const FTimespan FUdpMessageBeacon::MinimumInterval = FTimespan::FromMilliseconds(1000);


/* FUdpMessageHelloSender structors
 *****************************************************************************/

FUdpMessageBeacon::FUdpMessageBeacon(
	UE::UdpMessaging::FSocketSender* InSocketSender,
	const FGuid& InSocketId,
	const FIPv4Endpoint& InMulticastEndpoint
)
	: BeaconInterval(MinimumInterval)
	, LastEndpointCount(1)
	, LastHelloSent(FDateTime::MinValue())
	, NextHelloTime(FDateTime::UtcNow())
	, NodeId(InSocketId)
	, SocketSender(InSocketSender)
	, Stopping(false)
	, bSocketError(false)
{
	check(SocketSender);
	EndpointLeftEvent = FPlatformProcess::GetSynchEventFromPool(false);
	MulticastAddress = InMulticastEndpoint.ToInternetAddr();
	AddLocalEndpoints();

	Thread = FRunnableThread::Create(this, TEXT("FUdpMessageBeacon"), 128 * 1024, TPri_AboveNormal, FPlatformAffinity::GetPoolThreadMask());
}


FUdpMessageBeacon::~FUdpMessageBeacon()
{
	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
	}

	MulticastAddress = nullptr;
	StaticAddresses.Empty();

	FPlatformProcess::ReturnSynchEventToPool(EndpointLeftEvent);
	EndpointLeftEvent = nullptr;
}


/* FUdpMessageHelloSender interface
 *****************************************************************************/

void FUdpMessageBeacon::SetEndpointCount(int32 EndpointCount)
{
	check(EndpointCount > 0);

	if (EndpointCount < LastEndpointCount)
	{
		FDateTime CurrentTime = FDateTime::UtcNow();

		// adjust the send interval for reduced number of endpoints
		NextHelloTime = CurrentTime + (EndpointCount / LastEndpointCount) * (NextHelloTime - CurrentTime);
		LastHelloSent = CurrentTime - (EndpointCount / LastEndpointCount) * (CurrentTime - LastHelloSent);
		LastEndpointCount = EndpointCount;

		EndpointLeftEvent->Trigger();
	}
}


bool FUdpMessageBeacon::HasSocketError() const
{
	return bSocketError;
}

void FUdpMessageBeacon::AddStaticEndpoint(const FIPv4Endpoint& InEndpoint)
{
	FPendingEndpoint EndPt{InEndpoint,true};
	StaticEndpointQueue.Enqueue(MoveTemp(EndPt));
}

void FUdpMessageBeacon::RemoveStaticEndpoint(const FIPv4Endpoint& InEndpoint)
{
	FPendingEndpoint EndPt{InEndpoint, false};
	StaticEndpointQueue.Enqueue(MoveTemp(EndPt));
}

/* FRunnable interface
 *****************************************************************************/

FSingleThreadRunnable* FUdpMessageBeacon::GetSingleThreadInterface()
{
	return this;
}


bool FUdpMessageBeacon::Init()
{
	return true;
}


uint32 FUdpMessageBeacon::Run()
{
	while (!Stopping)
	{
		FDateTime CurrentTime = FDateTime::UtcNow();
		Update(CurrentTime, BeaconInterval);
		// Clamp the wait time to a positive value of at least 100ms.
		uint32 WaitTimeMs = (uint32)FMath::Max((NextHelloTime - CurrentTime).GetTotalMilliseconds(), 100.0);
		EndpointLeftEvent->Wait(WaitTimeMs);
	}

	SendSegment(EUdpMessageSegments::Bye, BeaconInterval);

	return 0;
}


void FUdpMessageBeacon::Stop()
{
	Stopping = true;
}


/* FUdpMessageHelloSender implementation
 *****************************************************************************/

bool FUdpMessageBeacon::SendSegment(EUdpMessageSegments SegmentType, const FTimespan& SocketWaitTime)
{
	FUdpMessageSegment::FHeader Header;
	{
		Header.SenderNodeId = NodeId;
		Header.ProtocolVersion = UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION;
		Header.SegmentType = SegmentType;
	}

	TSharedRef<FArrayWriter, ESPMode::ThreadSafe> Writer = MakeShared<FArrayWriter, ESPMode::ThreadSafe>();

	*Writer << Header;
	*Writer << NodeId;

	if (!SocketSender->Send(Writer, MulticastAddress))
	{
		bSocketError = true;
		return false;
	}

	return true;
}


bool FUdpMessageBeacon::SendPing(const FTimespan& SocketWaitTime)
{
	FUdpMessageSegment::FHeader Header;
	{
		Header.SenderNodeId = NodeId;
		// Pings were introduced at ProtocolVersion 11 and those messages needs to be send with that header to allow backward and forward discoverability
		Header.ProtocolVersion = 11;
		Header.SegmentType = EUdpMessageSegments::Ping;
	}
	uint8 ActualProtocolVersion = UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION;

	TSharedRef<FArrayWriter, ESPMode::ThreadSafe> Writer = MakeShared<FArrayWriter, ESPMode::ThreadSafe>();
	{
		*Writer << Header;
		*Writer << NodeId;
		*Writer << ActualProtocolVersion; // Send our actual Protocol version as part of the ping message
	}


	bool Result = true;
	for (const auto& StaticAddress : StaticAddresses)
	{
		if (!SocketSender->Send(Writer, StaticAddress))
		{	
			Result = false; // send failed
		}
	}
	return Result;
}


void FUdpMessageBeacon::Update(const FDateTime& CurrentTime, const FTimespan& SocketWaitTime)
{
	ProcessStaticEndpointQueue();

	if (CurrentTime < NextHelloTime)
	{
		return;
	}

	BeaconInterval = FMath::Max(MinimumInterval, IntervalPerEndpoint * LastEndpointCount);
	if (SendSegment(EUdpMessageSegments::Hello, SocketWaitTime))
	{
		NextHelloTime = CurrentTime + BeaconInterval;
	}
	SendPing(SocketWaitTime);
}

/* FSingleThreadRunnable interface
 *****************************************************************************/

void FUdpMessageBeacon::Tick()
{
	Update(FDateTime::UtcNow(), FTimespan::Zero());
}

/* Private
 *****************************************************************************/

void FUdpMessageBeacon::ProcessStaticEndpointQueue()
{
	FPendingEndpoint PendingEndpoint;
	while (StaticEndpointQueue.Dequeue(PendingEndpoint))
	{
		TSharedRef<FInternetAddr> Address = PendingEndpoint.StaticEndpoint.ToInternetAddr();
		if (PendingEndpoint.bAdd)
		{
			StaticAddresses.Add(Address);
		}
		else
		{
			StaticAddresses.RemoveAll([Address](TSharedPtr<FInternetAddr> Addr)
			{
				return Address->CompareEndpoints(*Addr.Get());
			});
		}
	}
}


void FUdpMessageBeacon::AddLocalEndpoints()
{
	// this is to properly discover other message bus process bound on the loopback address (i.e for the launcher not to trigger firewall)
	// note: this will only properly work if the socket is not bound (or bound to the any address)
	FIPv4Endpoint LocalEndpoint(FIPv4Address(127, 0, 0, 1), MulticastAddress->GetPort());
	{
		StaticAddresses.Add(LocalEndpoint.ToInternetAddr());
	}
}

