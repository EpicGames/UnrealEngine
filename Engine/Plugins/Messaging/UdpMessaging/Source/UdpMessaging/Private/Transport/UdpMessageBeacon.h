// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "Misc/SingleThreadRunnable.h"
#include "HAL/Runnable.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Containers/SpscQueue.h"

enum class EUdpMessageSegments : uint8;

class FEvent;
class FInternetAddr;
class FSocket;

namespace UE::UdpMessaging { class FSocketSender; }


/**
 * Implements a beacon sender thread.
 *
 */
class FUdpMessageBeacon
	: public FRunnable
	, private FSingleThreadRunnable
{
public:

	/** 
	 * Creates and initializes a new beacon sender.
	 *
	 * @param InSocketSender The network socket sender used to send Hello segments.
	 * @param InSocketId The network socket identifier (used to detect unicast endpoint).
	 * @param InMulticastEndpoint The multicast group endpoint to transport messages to.
	 */
	FUdpMessageBeacon(
		UE::UdpMessaging::FSocketSender* InSocketSender,
		const FGuid& InSocketId,
		const FIPv4Endpoint& InMulticastEndpoint
	);

	/** Virtual destructor. */
	virtual ~FUdpMessageBeacon();

public:

	/**
	 * Gets the current time interval between Hello segments.
	 *
	 * @return Beacon interval.
	 */
	FTimespan GetBeaconInterval()
	{
		return BeaconInterval;
	}

	/**
	 * Sets the number of known IP endpoints.
	 *
	 * @param EndpointCount The current number of known endpoints.
	 */
	void SetEndpointCount(int32 EndpointCount);

	/** 
	 * Get the socket error flag.
	 * @return if there were any socket error
	 */
	bool HasSocketError() const;

	/**
	 * Add a static endpoint to the beacon
	 * @param InEndpoint the endpoint to add
	 */
	void AddStaticEndpoint(const FIPv4Endpoint& InEndpoint);

	/**
	 * Remove a static endpoint from the beacon
	 * @param InEndpoint the endpoint to remove
	 */
	void RemoveStaticEndpoint(const FIPv4Endpoint& InEndpoint);

public:

	//~ FRunnable interface

	virtual FSingleThreadRunnable* GetSingleThreadInterface() override;
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override { }

protected:

	/**
	 * Sends the specified segment.
	 *
	 * @param SegmentType The type of segment to send (Hello or Bye).
	 * @param SocketWaitTime Maximum time to wait for the socket to be ready.
	 * @return true on success, false otherwise.
	 */
	bool SendSegment(EUdpMessageSegments SegmentType, const FTimespan& SocketWaitTime);

	/**
	 * Sends a ping segment to static addresses.
	 * @return true on success, false otherwise.
	 */
	bool SendPing(const FTimespan& SocketWaitTime);

	/**
	 * Update the beacon sender.
	 *
	 * @param CurrentTime The current time (in UTC).
	 * @param SocketWaitTime Maximum time to wait for the socket to be ready.
	 */
	void Update(const FDateTime& CurrentTime, const FTimespan& SocketWaitTime);

protected:

	//~ FSingleThreadRunnable interface

	virtual void Tick() override;

private:
	/** Add pending endpoints to the list of static addresses. */
	void ProcessStaticEndpointQueue();

	/** Add local static endpoints to discover other processes bound to other local network adapters. */
	void AddLocalEndpoints();

	/** Holds the calculated interval between Hello segments. */
	std::atomic<FTimespan> BeaconInterval;

	/** Holds an event signaling that an endpoint left. */
	FEvent* EndpointLeftEvent;

	/** Holds the number of known endpoints when NextHelloTime was last calculated. */
	int32 LastEndpointCount;

	/** Holds the time at which the last Hello segment was sent. */
	FDateTime LastHelloSent;

	/** Holds the multicast address and port number to send to. */
	TSharedPtr<FInternetAddr> MulticastAddress;

	/** Pending static endpoint to send to the beacon. */
	struct FPendingEndpoint
	{
		FIPv4Endpoint StaticEndpoint;
		bool bAdd;
	};

	/** Static endpoint queue to add/remove static endpoints. */
	TSpscQueue<FPendingEndpoint> StaticEndpointQueue;

	/** Holds the static addresses to broadcast ping to. */
	TArray<TSharedPtr<FInternetAddr>> StaticAddresses;

	/** Holds the time at which the next Hello segment must be sent. */
	FDateTime NextHelloTime;

	/** Holds local node identifier. */
	FGuid NodeId;

	/** Holds the socket sender used to send Hello segments. */
	UE::UdpMessaging::FSocketSender* SocketSender;

	/** Holds a flag indicating that the thread is stopping. */
	bool Stopping;

	/** Holds a flag indicating if we encountered socket sending error. */
	bool bSocketError;

	/** Holds the thread object. */
	FRunnableThread* Thread;
private:
	
	/** Defines the time interval per endpoint. */
	static const FTimespan IntervalPerEndpoint;

	/** Defines the minimum interval for Hello segments. */
	static const FTimespan MinimumInterval;
};
