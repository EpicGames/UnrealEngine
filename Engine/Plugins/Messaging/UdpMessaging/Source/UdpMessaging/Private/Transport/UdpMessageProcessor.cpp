// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transport/UdpMessageProcessor.h"
#include "Algo/AllOf.h"

#include "CoreGlobals.h"
#include "HAL/RunnableThread.h"
#include "Misc/ConfigCacheIni.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"

#include "Transport/UdpDeserializedMessage.h"
#include "UdpMessagingPrivate.h"
#include "Shared/SocketSender.h"
#include "Shared/UdpMessagingSettings.h"
#include "Trace/UdpMessagingTrace.h"
#include "Transport/UdpMessageBeacon.h"
#include "Transport/UdpMessageSegmenter.h"
#include "Transport/UdpReassembledMessage.h"
#include "Transport/UdpSerializedMessage.h"
#include "Transport/UdpSerializeMessageTask.h"
#include "Shared/UdpMessageSegment.h"

#include "Interfaces/IPluginManager.h"

/* FUdpMessageHelloSender static initialization
 *****************************************************************************/

#if UDPMESSAGINGTRACE_ENABLED
std::atomic<uint16> FUdpMessageProcessor::NextNodeTraceId(1);
#endif

const int32 FUdpMessageProcessor::DeadHelloIntervals = 5;

TAutoConsoleVariable<int32> CVarFakeSocketError(
	TEXT("MessageBus.UDP.InduceSocketError"),
	0,
	TEXT("This CVar can be used to induce a socket failure on outbound communication.\n")
	TEXT("Any non zero value will force the output socket connection to fail if the IP address matches\n")
	TEXT("one of the values in MessageBus.UDP.ConnectionsToError. The list can be cleared by invoking\n")
	TEXT("MessageBus.UDP.ClearDenyList."),
	ECVF_Default
);

TAutoConsoleVariable<FString> CVarConnectionsToError(
	TEXT("MessageBus.UDP.ConnectionsToError"),
	TEXT(""),
	TEXT("Connections to error out on when MessageBus.UDP.InduceSocketError is enabled.\n")
	TEXT("This can be a comma separated list in the form IPAddr2:port,IPAddr3:port"),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarUdpMessagingMaxWindowSize(
	TEXT("MessageBus.UDP.MaxWindowSize"),
	2048,
	TEXT("Maximum window size for sending to endpoints."),
	ECVF_Default);

TAutoConsoleVariable<int32> CVarUdpMessagingMaxOffloadSegments(
	TEXT("MessageBus.UDP.MaxOffloadSegments"),
	16,
	TEXT("Maximum number of datagrams to generate per send call using UDP segmentation offload. Maximum value is 64."),
	ECVF_Default);


namespace UE::Private::MessageProcessor
{

bool ShouldErrorOnConnection(const FIPv4Endpoint& InEndpoint)
{
	if (CVarFakeSocketError.GetValueOnAnyThread() == 0)
	{
		return false;
	}

	const FString ConnectionsToErrorString = CVarConnectionsToError.GetValueOnAnyThread();
	TArray<FString> EndpointStrings;
	ConnectionsToErrorString.ParseIntoArray(EndpointStrings, TEXT(","));
	for (const FString& EndpointString : EndpointStrings)
	{
		FIPv4Endpoint Endpoint;
		if (FIPv4Endpoint::Parse(EndpointString, Endpoint))
		{
			if (Endpoint.Address == InEndpoint.Address &&
				(Endpoint.Port == 0 || Endpoint.Port == InEndpoint.Port))
			{
				return true;
			}
		}
	}
	return false;
}

FOnOutboundTransferDataUpdated& OnSegmenterUpdated()
{
	static FOnOutboundTransferDataUpdated OnTransferUpdated;
	return OnTransferUpdated;
}

FOnInboundTransferDataUpdated& OnReassemblerUpdated()
{
	static FOnInboundTransferDataUpdated OnTransferUpdated;
	return OnTransferUpdated;
}

}

/* FUdpMessageProcessor structors
 *****************************************************************************/
FUdpMessageProcessor::FNodeInfo::FNodeInfo()
	: NodeId()
	, ProtocolVersion(UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION)
	, ReliableWorkQueue(1024)
	, UnreliableWorkQueue(1024)
{
	ComputeWindowSize(0, 0);
}

FUdpMessageProcessor::FUdpMessageProcessor(FSocket& InSocket, const FGuid& InNodeId, const FIPv4Endpoint& InMulticastEndpoint)
	: Beacon(nullptr)
	, LocalNodeId(InNodeId)
	, LastSentMessage(-1)
	, MulticastEndpoint(InMulticastEndpoint)
	, Socket(&InSocket)
	, SocketSender(nullptr)
	, bStopping(false)
	, bIsInitialized(false)
	, MessageFormat(GetDefault<UUdpMessagingSettings>()->MessageFormat) // NOTE: When the message format changes (in the Udp Messaging settings panel), the service is restarted and the processor recreated.
{
	Init();

	MemoryLoggingThresholdMB = GConfig->GetFloatOrDefault(TEXT("/Script/UdpMessaging.UdpMessagingSettings"), TEXT("NodeMemoryLoggingThresholdMB"), -1, GEngineIni);

	WorkEvent = MakeShareable(FPlatformProcess::GetSynchEventFromPool(), [](FEvent* EventToDelete)
	{
		FPlatformProcess::ReturnSynchEventToPool(EventToDelete);
	});

	const ELoadingPhase::Type LoadingPhase = IPluginManager::Get().GetLastCompletedLoadingPhase();
	if (LoadingPhase == ELoadingPhase::None || LoadingPhase < ELoadingPhase::PostDefault)
	{
		IPluginManager::Get().OnLoadingPhaseComplete().AddRaw(this, &FUdpMessageProcessor::OnPluginLoadingPhaseComplete);
	}
	else
	{
		StartThread();
	}
}


FUdpMessageProcessor::~FUdpMessageProcessor()
{
	// shut down worker thread if it is still running
	if (Thread)
	{
		Thread->Kill();
	}

	Thread = {};
	Beacon = {};

	// NOTE: Socket sender must be destroyed after the beacon because the beacon uses the socket sender.
	SocketSender = {};

	IPluginManager::Get().OnLoadingPhaseComplete().RemoveAll(this);

	// remove all transport nodes
	if (NodeLostDelegate.IsBound())
	{
		for (auto& KnownNodePair : KnownNodes)
		{
			NodeLostDelegate.Execute(KnownNodePair.Key);
		}
	}

	KnownNodes.Empty();
}

void FUdpMessageProcessor::StartThread()
{
	Thread = TUniquePtr<FRunnableThread>(FRunnableThread::Create(this, TEXT("FUdpMessageProcessor"), 128 * 1024, TPri_AboveNormal, FPlatformAffinity::GetPoolThreadMask()));
}

void FUdpMessageProcessor::OnPluginLoadingPhaseComplete(ELoadingPhase::Type LoadingPhase, bool bPhaseSuccessful)
{
	check(!Thread);
	if (LoadingPhase == ELoadingPhase::PostDefault)
	{
		IPluginManager::Get().OnLoadingPhaseComplete().RemoveAll(this);
		StartThread();
	}
}

void FUdpMessageProcessor::AddStaticEndpoint(const FIPv4Endpoint& InEndpoint)
{
	if (Beacon)
	{
		Beacon->AddStaticEndpoint(InEndpoint);
	}
}


void FUdpMessageProcessor::RemoveStaticEndpoint(const FIPv4Endpoint& InEndpoint)
{
	if (Beacon)
	{
		Beacon->RemoveStaticEndpoint(InEndpoint);
	}
}

TArray<FIPv4Endpoint> FUdpMessageProcessor::GetKnownEndpoints() const
{
	TArray<FIPv4Endpoint> Endpoints;
	for (const auto& NodePair : KnownNodes)
	{
		Endpoints.Add(NodePair.Value.Endpoint);
	}
	return Endpoints;
}

/* FUdpMessageProcessor interface
 *****************************************************************************/

TMap<uint8, TArray<FGuid>> FUdpMessageProcessor::GetRecipientsPerProtocolVersion(const TArray<FGuid>& Recipients)
{
	TMap<uint8, TArray<FGuid>> NodesPerVersion;
	{
		FScopeLock NodeVersionsLock(&NodeVersionCS);

		// No recipients means a publish, so broadcast to all known nodes (static nodes are in known nodes.)
		// We used to broadcast on the multicast endpoint, but the discovery of nodes should have found available nodes using multicast already
		if (Recipients.Num() == 0)
		{
			for (auto& NodePair : NodeVersions)
			{
				NodesPerVersion.FindOrAdd(NodePair.Value).Add(NodePair.Key);
			}
		}
		else
		{
			for (const FGuid& Recipient : Recipients)
			{
				uint8* Version = NodeVersions.Find(Recipient);
				if (Version)
				{
					NodesPerVersion.FindOrAdd(*Version).Add(Recipient);
				}
			}
		}
	}
	return NodesPerVersion;
}

bool FUdpMessageProcessor::EnqueueInboundSegment(const TSharedPtr<FArrayReader, ESPMode::ThreadSafe>& Data, const FIPv4Endpoint& InSender)
{
	if (bStopping)
	{
		return false;
	}

	InboundSegments.Enqueue(Data, InSender, FUdpMessagingTime::Now());

	WorkEvent->Trigger();

	return true;
}

bool FUdpMessageProcessor::EnqueueOutboundMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext, const TArray<FGuid>& Recipients)
{
	if (bStopping)
	{
		return false;
	}

	TMap<uint8, TArray<FGuid>> RecipientPerVersions = GetRecipientsPerProtocolVersion(Recipients);
	for (const auto& RecipientVersion : RecipientPerVersions)
	{
		const FUdpMessagingTime EnqueueTime = FUdpMessagingTime::Now();

		// Create a message to serialize using that protocol version
		TSharedRef<FUdpSerializedMessage, ESPMode::ThreadSafe> SerializedMessage = MakeShared<FUdpSerializedMessage, ESPMode::ThreadSafe>(MessageFormat, RecipientVersion.Key, MessageContext->GetFlags());

		// Kick off the serialization task
		TGraphTask<FUdpSerializeMessageTask>::CreateTask().ConstructAndDispatchWhenReady(MessageContext, SerializedMessage, WorkEvent);

		// Enqueue the message
		OutboundMessages.Enqueue(SerializedMessage, RecipientVersion.Value, MessageContext->GetFlags(), EnqueueTime);
	}

	return true;
}

FMessageTransportStatistics FUdpMessageProcessor::GetStats(FGuid Node) const
{
	FScopeLock NodeVersionLock(&StatisticsCS);
	if (FMessageTransportStatistics const* Stats = NodeStats.Find(Node))
	{
		return *Stats;
	}
	return {};
}

void FUdpMessageProcessor::SendSegmenterStatsToListeners(int32 MessageId, FGuid NodeId, const TSharedPtr<FUdpMessageSegmenter>& Segmenter)
{
	FOnOutboundTransferDataUpdated& SegmenterUpdatedDelegate = UE::Private::MessageProcessor::OnSegmenterUpdated();
	if(!SegmenterUpdatedDelegate.IsBound())
	{
		return;
	}

	if (!Segmenter->IsInitialized() || Segmenter->IsInvalid())
	{
		return;
	}

	const uint16 MaxSegmentSize = Segmenter->GetTotalSegmentSize();
	const uint32 SegmentCount = Segmenter->GetSegmentCount();
	SegmenterUpdatedDelegate.Broadcast(
		{
			NodeId,
			MessageId,
			// Convert segment data into bytes.
			MaxSegmentSize * SegmentCount,
			MaxSegmentSize * (SegmentCount - Segmenter->GetPendingSendSegmentsCount()),
			MaxSegmentSize * (Segmenter->GetAcknowledgedSegmentsCount()),
			EnumHasAnyFlags(Segmenter->GetMessageFlags(), EMessageFlags::Reliable),
			EnumHasAnyFlags(Segmenter->GetMessageFlags(), EMessageFlags::Traced),
			Segmenter->AreAcknowledgementsComplete()
		});
}

void FUdpMessageProcessor::UpdateNetworkStatistics()
{
	FScopeLock NodeVersionLock(&StatisticsCS);
	NodeStats.Reset();
	for (const auto& NodePair : KnownNodes)
	{
		NodeStats.Add(NodePair.Key, NodePair.Value.Statistics);
	}
}

/* FRunnable interface
 *****************************************************************************/

FSingleThreadRunnable* FUdpMessageProcessor::GetSingleThreadInterface()
{
	return this;
}


bool FUdpMessageProcessor::Init()
{
	if (!bIsInitialized)
	{
		const UUdpMessagingSettings* Settings = GetDefault<UUdpMessagingSettings>();

		UE::UdpMessaging::FSocketSender::FOptions SenderOpts;

		SenderOpts.SegmentOffloadSize =
			Settings->bSegmentationOffload
			? (Settings->GetMaxPacketSize() - UE::UdpMessaging::PacketHeaderBytes)
			: 0;

		const uint64 OneGbitPerSecondInBytes = 125000000;
		SenderOpts.BytesPerSec = GetDefault<UUdpMessagingSettings>()->GetMaxSendRate() * OneGbitPerSecondInBytes;
		SenderOpts.MaxBurstBytes = SenderOpts.BytesPerSec * 0.1; // Absorbs 100ms jitter

		SocketSender = MakeUnique<UE::UdpMessaging::FSocketSender>(Socket, TEXT("FUdpMessageProcessor.Sender"), SenderOpts);

		Beacon = MakeUnique<FUdpMessageBeacon>(SocketSender.Get(), LocalNodeId, MulticastEndpoint);

#if UDPMESSAGINGTRACE_ENABLED
		LocalNodeTraceId = NextNodeTraceId++;
#endif

		TRACE_UDPMESSAGING_DISCOVERED_NODE(
			LocalNodeTraceId, 0, LocalNodeId, FIPv4Endpoint(SocketSender->GetAddress()), UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION
		);

		// Current protocol version 18
		SupportedProtocolVersions.Add(UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION);
		// Support Protocol version 10, 11, 12, 13, 14, 15, 16, 17
		SupportedProtocolVersions.Add(17);
		SupportedProtocolVersions.Add(16);
		SupportedProtocolVersions.Add(15);
		SupportedProtocolVersions.Add(14);
		SupportedProtocolVersions.Add(13);
		SupportedProtocolVersions.Add(12);
		SupportedProtocolVersions.Add(11);
		SupportedProtocolVersions.Add(10);
		bIsInitialized = true;
	}
	return bIsInitialized;
}


uint32 FUdpMessageProcessor::Run()
{
	FUdpMessagingTime LastTime = FUdpMessagingTime::Now();

	while (!bStopping)
	{
		WorkEvent->Wait(CalculateWaitTime());

		SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_Run);

		do
		{
			SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_Run_Frame);

			const FUdpMessagingTime Now = FUdpMessagingTime::Now();
			DeltaTime = Now - LastTime;
			LastTime = Now;

			for (TSet<FUdpReassembledMessageSharedPtr>::TIterator It = PendingTypeLookup.CreateIterator(); It; ++It)
			{
				if (LookupAndCacheMessageType(*It))
				{
					It.RemoveCurrent();
				}
			}

			ConsumeInboundSegments();
			ConsumeOutboundMessages();
			UpdateKnownNodes();
			UpdateNetworkStatistics();
		} while ((!InboundSegments.IsEmpty() || MoreToSend()) && !bStopping);
	}

	return 0;
}


void FUdpMessageProcessor::Stop()
{
	bStopping = true;
	WorkEvent->Trigger();
}


void FUdpMessageProcessor::WaitAsyncTaskCompletion()
{
	// Stop to prevent any new work from being queued.
	Stop();

	// Wait for the processor thread, so we can access KnownNodes safely
	if (Thread)
	{
		Thread->WaitForCompletion();
	}

	// Check if processor has in-flight serialization task(s).
	auto HasIncompleteSerializationTasks = [this]()
	{
		for (const TPair<FGuid, FNodeInfo>& GuidNodeInfoPair : KnownNodes)
		{
			for (const TPair<int32, TSharedPtr<FUdpMessageSegmenter>>& SegmenterPair: GuidNodeInfoPair.Value.Segmenters)
			{
				if (!SegmenterPair.Value->IsMessageSerializationDone())
				{
					return true;
				}
			}
		}

		return false;
	};

	// Ensures the task graph doesn't contain any pending/running serialization tasks after the processor exit. If the engine is shutting down, the serialization (UStruct) might
	// not be available anymore when the task is run (The task graph shuts down after the UStruct stuff).
	while (HasIncompleteSerializationTasks())
	{
		FPlatformProcess::Sleep(0); // Yield.
	}
}

/* FSingleThreadRunnable interface
*****************************************************************************/

void FUdpMessageProcessor::Tick()
{
	ConsumeInboundSegments();
	ConsumeOutboundMessages();
	UpdateKnownNodes();
	UpdateNetworkStatistics();
}

/* FUdpMessageProcessor implementation
 *****************************************************************************/

void FUdpMessageProcessor::AcknowledgeReceipt(int32 MessageId, const FNodeInfo& NodeInfo)
{
	FUdpMessageSegment::FHeader Header;
	{
		Header.ProtocolVersion = NodeInfo.ProtocolVersion;
		Header.RecipientNodeId = NodeInfo.NodeId;
		Header.SenderNodeId = LocalNodeId;
		Header.SegmentType = EUdpMessageSegments::Acknowledge;
	}

	FUdpMessageSegment::FAcknowledgeChunk AcknowledgeChunk;
	{
		AcknowledgeChunk.MessageId = MessageId;
	}

	TSharedRef<FArrayWriter, ESPMode::ThreadSafe> Writer = MakeShared<FArrayWriter, ESPMode::ThreadSafe>();
	*Writer << Header;
	AcknowledgeChunk.Serialize(*Writer, NodeInfo.ProtocolVersion);

	const bool bAllowSegmentation_false = false;
	const bool bHighPriority_true = true;
	if (!SocketSender->Send(Writer, NodeInfo.Endpoint, bAllowSegmentation_false, bHighPriority_true))
	{
		UE_LOGF(LogUdpMessaging, Warning, "FUdpMessageProcessor::AcknowledgeReceipt failed to send %ls.", *NodeInfo.Endpoint.ToString());
		return;
	}

	UE_LOGF(LogUdpMessaging, Verbose, "Sending EUdpMessageSegments::Acknowledge for msg %d from %ls", MessageId, *NodeInfo.NodeId.ToString());
}


FTimespan FUdpMessageProcessor::CalculateWaitTime() const
{
	// TODO: Peek at expiry queues to determine when it would be appropriate to wake sooner; potentially increase this?
	return FTimespan::FromMilliseconds(10);
}


void FUdpMessageProcessor::ConsumeInboundSegments()
{
	SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_ConsumeInboundSegments);

	const FTimespan MaxWorkTimespan = FTimespan::FromMilliseconds(5);
	const FUdpMessagingTime EndWorkTime = FUdpMessagingTime::Now() + MaxWorkTimespan;

	while (TOptional<FInboundSegment> SegmentOptional = InboundSegments.Dequeue())
	{
		FInboundSegment& Segment = SegmentOptional.GetValue();

		// quick hack for TTP# 247103
		if (!Segment.Data.IsValid())
		{
			continue;
		}

		FUdpMessageSegment::FHeader Header;
		*Segment.Data << Header;

		const bool bPassesFilter = FilterSegment(Header, Segment.Sender);
		if (bEnableMessageDelegates)
		{
			InboundSegmentDelegate.Broadcast(Segment, Header.SenderNodeId, Header.SegmentType, bPassesFilter);
		}

		if (bPassesFilter)
		{
			FNodeInfo& NodeInfo = KnownNodes.FindOrAdd(Header.SenderNodeId);

			if (!NodeInfo.NodeId.IsValid())
			{
#if UDPMESSAGINGTRACE_ENABLED
				NodeInfo.NodeTraceId = NextNodeTraceId++;
#endif
				NodeInfo.NodeId = Header.SenderNodeId;
				NodeInfo.ProtocolVersion = Header.ProtocolVersion;

				TRACE_UDPMESSAGING_DISCOVERED_NODE(NodeInfo.NodeTraceId, LocalNodeTraceId,
					NodeInfo.NodeId, Segment.Sender, NodeInfo.ProtocolVersion);

				const uint16 MaxSegmentSizeV10_V17 = 2048 - UE::UdpMessaging::PacketHeaderBytes;
				const uint16 MaxSegmentSize = GetDefault<UUdpMessagingSettings>()->GetMaxPacketSize()
					- UE::UdpMessaging::PacketHeaderBytes;

				NodeInfo.MaxSegmentSize = (Header.ProtocolVersion >= 18)
					? MaxSegmentSize
					: MaxSegmentSizeV10_V17;

				// If USO is enabled, we need to clamp to that size here.
				if (const uint16 SegmentOffloadPayloadSize = SocketSender->GetSegmentOffloadSize())
				{
					NodeInfo.MaxSegmentSize = FMath::Min(NodeInfo.MaxSegmentSize, SegmentOffloadPayloadSize);
				}

				NodeDiscoveredDelegate.ExecuteIfBound(NodeInfo.NodeId);
				bAddedNewKnownNodes = true;
			}

			NodeInfo.Endpoint = Segment.Sender;
			NodeInfo.LastSegmentReceivedTime = FUdpMessagingTime::Now();
			NodeInfo.MaxWindowSize = CVarUdpMessagingMaxWindowSize.GetValueOnAnyThread();
			
			switch (Header.SegmentType)
			{
			case EUdpMessageSegments::Abort:
				ProcessAbortSegment(Segment, NodeInfo);
				break;

			case EUdpMessageSegments::Acknowledge:
				ProcessAcknowledgeSegment(Segment, NodeInfo);
				break;

			case EUdpMessageSegments::AcknowledgeSegments:
				ProcessAcknowledgeSegmentsSegment(Segment, NodeInfo);
				break;

			case EUdpMessageSegments::Bye:
				ProcessByeSegment(Segment, NodeInfo);
				break;

			case EUdpMessageSegments::Data:
				ProcessDataSegment(Segment, NodeInfo);
				break;

			case EUdpMessageSegments::Hello:
				ProcessHelloSegment(Segment, NodeInfo);
				break;

			case EUdpMessageSegments::Ping:
				ProcessPingSegment(Segment, NodeInfo);
				break;

			case EUdpMessageSegments::Pong:
				ProcessPongSegment(Segment, NodeInfo);
				break;

			case EUdpMessageSegments::Retransmit:
				ProcessRetransmitSegment(Segment, NodeInfo);
				break;

			case EUdpMessageSegments::Timeout:
				ProcessTimeoutSegment(Segment, NodeInfo);
				break;

			case EUdpMessageSegments::Mesh:
				ProcessMeshSegment(Segment, NodeInfo);
				break;

			default:
				ProcessUnknownSegment(Segment, NodeInfo, (uint8)Header.SegmentType);
			}
		}

		if (FUdpMessagingTime::Now() >= EndWorkTime)
		{
			break;
		}
	}
}

int32 FUdpMessageProcessor::AssignNextMessageId()
{
	++LastSentMessage;
	if (LastSentMessage < 0)
	{
		// Prevent negative message ids
		LastSentMessage = 0;
	}
	return LastSentMessage;
}

void FUdpMessageProcessor::ConsumeOneOutboundMessage(const FOutboundMessage& OutboundMessage)
{
	const auto FindNodeWithLog = [this](const FGuid &Id)
	{
		FNodeInfo *NodeInfo = KnownNodes.Find(Id);
		if (NodeInfo == nullptr)
		{
			UE_LOGF(LogUdpMessaging, Verbose, "No recipient NodeInfo found for %ls", *Id.ToString());
		}
		return NodeInfo;
	};

	TArray<FNodeInfo*> Recipients;
	Algo::TransformIf(OutboundMessage.RecipientIds, Recipients,
					  [FindNodeWithLog](const FGuid &Id) {return FindNodeWithLog(Id);},
					  [this](const FGuid &Id) {return KnownNodes.Find(Id);} );

	const bool bIsReliable = EnumHasAnyFlags(OutboundMessage.MessageFlags, EMessageFlags::Reliable);

	const int32 MessageId = AssignNextMessageId();

	for (FNodeInfo* RecipientNodeInfo : Recipients)
	{
		TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
			OutboundMessage.EnqueueTime.Cycles,
			LocalNodeTraceId,
			RecipientNodeInfo->NodeTraceId,
			MessageId,
			UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::EnqueueOutbound
		);

		TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
			FPlatformTime::Cycles64(),
			LocalNodeTraceId,
			RecipientNodeInfo->NodeTraceId,
			MessageId,
			UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::ConsumeOutbound
		);

		UE_LOG(LogUdpMessaging, VeryVerbose, TEXT("Passing %" INT64_FMT " byte message to be segment-sent %d to %s with id %s"),
				OutboundMessage.SerializedMessage->TotalSize(),
				MessageId,
				*RecipientNodeInfo->Endpoint.ToString(),
				*RecipientNodeInfo->NodeId.ToString());

		if (!bIsReliable && !RecipientNodeInfo->CanSendSegments())
		{
			// Discard unreliable messages that cannot be sent.
			TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
				FPlatformTime::Cycles64(),
				LocalNodeTraceId,
				RecipientNodeInfo->NodeTraceId,
				MessageId,
				UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::Discarded
			);
			continue;
		}

		TSharedPtr<FUdpMessageSegmenter> Segmenter = MakeShared<FUdpMessageSegmenter>(
			OutboundMessage.SerializedMessage.ToSharedRef(), RecipientNodeInfo->MaxSegmentSize);

		RecipientNodeInfo->Segmenters.Add(MessageId, Segmenter);
		if (bIsReliable)
		{
			RecipientNodeInfo->ReliableWorkQueue.Add(MessageId);
		}
		else
		{
			RecipientNodeInfo->UnreliableWorkQueue.Add(MessageId);
		}
	}
}

void FUdpMessageProcessor::ConsumeOutboundMessages()
{
	SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_ConsumeOutputMessages);
	
	while (TOptional<FOutboundMessage> OutboundMessageOptional = OutboundMessages.Dequeue())
	{
		const FOutboundMessage& OutboundMessage = OutboundMessageOptional.GetValue();
		ConsumeOneOutboundMessage(OutboundMessage);
	}
}

bool FUdpMessageProcessor::FilterSegment(const FUdpMessageSegment::FHeader& Header, const FIPv4Endpoint& Sender)
{
	// filter locally generated segments
	if (Header.SenderNodeId == LocalNodeId)
	{
		return false;
	}

	if (!CanAcceptEndpointDelegate.Execute(Header.SenderNodeId, Sender))
	{
		return false;
	}

	// filter unsupported protocol versions
	if (!SupportedProtocolVersions.Contains(Header.ProtocolVersion))
	{
		return false;
	}

	return true;
}


void FUdpMessageProcessor::ProcessAbortSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo)
{
	SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_ProcessAbortSegment);

	FUdpMessageSegment::FAbortChunk AbortChunk;
	AbortChunk.Serialize(*Segment.Data, NodeInfo.ProtocolVersion);

	const int32 NumRemoved = NodeInfo.Segmenters.Remove(AbortChunk.MessageId);

	if (NumRemoved)
	{
		TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
			FPlatformTime::Cycles64(),
			LocalNodeTraceId,
			NodeInfo.NodeTraceId,
			AbortChunk.MessageId,
			UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::Discarded
		);
	}
}


void FUdpMessageProcessor::ProcessAcknowledgeSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo)
{
	SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_ProcessAcknowledgeSegment);

	FUdpMessageSegment::FAcknowledgeChunk AcknowledgeChunk;
	AcknowledgeChunk.Serialize(*Segment.Data, NodeInfo.ProtocolVersion);

	TSharedPtr<FUdpMessageSegmenter>* FoundSegmenter = NodeInfo.Segmenters.Find(AcknowledgeChunk.MessageId);
	if (FoundSegmenter)
	{
		TSharedPtr<FUdpMessageSegmenter>& Segmenter = *FoundSegmenter;
		if (EnumHasAnyFlags(Segmenter->GetMessageFlags(), EMessageFlags::Reliable))
		{
			NodeInfo.MarkComplete(AcknowledgeChunk.MessageId, FUdpMessagingTime::Now());
		}
		SendSegmenterStatsToListeners(AcknowledgeChunk.MessageId, NodeInfo.NodeId, Segmenter);
	}

	const int32 NumRemoved = NodeInfo.Segmenters.Remove(AcknowledgeChunk.MessageId);
	if (NumRemoved)
	{
		TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
			FPlatformTime::Cycles64(),
			LocalNodeTraceId,
			NodeInfo.NodeTraceId,
			AcknowledgeChunk.MessageId,
			UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::Complete
		);
	}

	UE_LOGF(LogUdpMessaging, Verbose, "Received Acknowledge for %d from %ls", AcknowledgeChunk.MessageId , *NodeInfo.NodeId.ToString());
}


void FUdpMessageProcessor::ProcessAcknowledgeSegmentsSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo) // TODO: Rename function
{
	SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_ProcessAcknowledgeSegmentsSegment);

	FUdpMessageSegment::FAcknowledgeSegmentsChunk AcknowledgeChunk;
	AcknowledgeChunk.Serialize(*Segment.Data, NodeInfo.ProtocolVersion);

	UE_LOGF(LogUdpMessaging, Verbose, "Received AcknowledgeSegments for %d from %ls", AcknowledgeChunk.MessageId, *NodeInfo.NodeId.ToString());
	const bool bIsComplete = NodeInfo.MarkAcks(AcknowledgeChunk.MessageId, AcknowledgeChunk.Segments, FUdpMessagingTime::Now());
	if (bIsComplete)
	{
		TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
			FPlatformTime::Cycles64(),
			LocalNodeTraceId,
			NodeInfo.NodeTraceId,
			AcknowledgeChunk.MessageId,
			UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::Complete
		);
	}
}


void FUdpMessageProcessor::ProcessByeSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo)
{
	SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_ProcessByeSegment);

	FGuid RemoteNodeId;
	*Segment.Data << RemoteNodeId;

	if (RemoteNodeId.IsValid() && (RemoteNodeId == NodeInfo.NodeId))
	{
		RemoveKnownNode(RemoteNodeId);
	}
}


void FUdpMessageProcessor::ProcessDataSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo)
{
	SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_ProcessDataSegment);

	FUdpMessageSegment::FDataChunk DataChunk;
	DataChunk.Serialize(*Segment.Data, NodeInfo.ProtocolVersion);

	if (Segment.Data->IsError())
	{
		UE_LOGF(LogUdpMessaging, Warning, "FUdpMessageProcessor::ProcessDataSegment: Failed to serialize DataChunk. Sender=%ls",
			*(Segment.Sender.ToString()));
		return;
	}

	if (NodeInfo.RecentlyDeliveredMessages.Contains(DataChunk.MessageId))
	{
		UE_LOGF(LogUdpMessaging, VeryVerbose, "FUdpMessageProcessor::ProcessDataSegment: Ignoring segment for recently delivered message %d",
			DataChunk.MessageId);
		return;
	}

	TSharedPtr<FUdpReassembledMessage>& ReassembledMessage = NodeInfo.ReassembledMessages.FindOrAdd(DataChunk.MessageId);

	// Reassemble message
	if (!ReassembledMessage.IsValid())
	{
		ReassembledMessage = MakeShared<FUdpReassembledMessage>(
			NodeInfo.ProtocolVersion,
			DataChunk.MessageFlags,
			DataChunk.MessageSize,
			DataChunk.TotalSegments,
			DataChunk.Sequence,
			Segment.Sender
		);

		if (ReassembledMessage->IsMalformed())
		{
			// Go ahead and throw away the message.
			// The sender should see the NAK and resend, so we'll attempt to recreate it later.
			UE_LOGF(LogUdpMessaging, Warning, "FUdpMessageProcessor::ProcessDataSegment: Ignoring malformed Message %ls", *(ReassembledMessage->Describe()));
			NodeInfo.ReassembledMessages.Remove(DataChunk.MessageId);
			ReassembledMessage.Reset();
			return;
		}

#if UDPMESSAGINGTRACE_ENABLED
		ReassembledMessage->GetTraceMetadata() = { .SenderShortId = NodeInfo.NodeTraceId, .RecipientShortId = LocalNodeTraceId, .MessageId = DataChunk.MessageId };
#endif

		TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(Segment.EnqueueTime.Cycles, NodeInfo.NodeTraceId, LocalNodeTraceId, DataChunk.MessageId, UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::FirstSegmentEnqueueInbound);
		TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(FPlatformTime::Cycles64(), NodeInfo.NodeTraceId, LocalNodeTraceId, DataChunk.MessageId, UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::BeginReassembly);
		TRACE_UDPMESSAGING_MESSAGE_SUMMARY(NodeInfo.NodeTraceId, LocalNodeTraceId, DataChunk.MessageId, DataChunk.MessageSize, DataChunk.TotalSegments, DataChunk.MessageFlags);
	}

	NodeInfo.Statistics.TotalBytesReceived += DataChunk.Data.Num();
	NodeInfo.Statistics.PacketsReceived++;
	ReassembledMessage->Reassemble(DataChunk.SegmentNumber, DataChunk.SegmentOffset, DataChunk.Data, FUdpMessagingTime::Now());

	if (DataChunk.SegmentNumber == 0)
	{
		if (!LookupAndCacheMessageType(ReassembledMessage))
		{
			PendingTypeLookup.Add(ReassembledMessage);
		}
	}

	FOnInboundTransferDataUpdated& ReassemblerUpdated = UE::Private::MessageProcessor::OnReassemblerUpdated();
	if(ReassemblerUpdated.IsBound())
	{
		ReassemblerUpdated.Broadcast(
			{
				NodeInfo.NodeId,
				DataChunk.MessageId,
				NodeInfo.MaxSegmentSize * ReassembledMessage->GetPendingSegmentsCount(),
				ReassembledMessage->GetReceivedBytes(),
				ReassembledMessage->IsReliable(),
				EnumHasAnyFlags(ReassembledMessage->GetFlags(), EMessageFlags::Traced),
				ReassembledMessage->IsComplete()
			}
		);
	}

	// Deliver or re-sequence message
	if (!ReassembledMessage->IsComplete() || ReassembledMessage->IsPendingDelivery() || ReassembledMessage->IsDelivered())
	{
		return;
	}

	TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(FPlatformTime::Cycles64(), NodeInfo.NodeTraceId, LocalNodeTraceId, DataChunk.MessageId, UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::EndReassembly);

	UE_LOGF(LogUdpMessaging, Verbose, "FUdpMessageProcessor::ProcessDataSegment: Reassembled %d bytes message %ls with %d for %ls (%ls)",
		ReassembledMessage->GetData().Num(),
		*ReassembledMessage->Describe(),
		DataChunk.MessageId,
		*NodeInfo.NodeId.ToString(),
		*NodeInfo.Endpoint.ToString());

	AcknowledgeReceipt(DataChunk.MessageId, NodeInfo);
	TryDeliverMessage(ReassembledMessage, NodeInfo);
}


void FUdpMessageProcessor::ProcessHelloSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo)
{
	FGuid RemoteNodeId;
	*Segment.Data << RemoteNodeId;

	ensure(RemoteNodeId == NodeInfo.NodeId);
}

void FUdpMessageProcessor::ProcessPingSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo)
{
	SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_ProcessPingSegment);

	FGuid RemoteNodeId;
	*Segment.Data << RemoteNodeId;
	uint8 NodeProtocolVersion;
	*Segment.Data << NodeProtocolVersion;

	ensure(RemoteNodeId == NodeInfo.NodeId);

	// The protocol version we are going to use to communicate to this node is the smallest between its version and our own
	uint8 ProtocolVersion = FMath::Min<uint8>(NodeProtocolVersion, UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION);

	// if that protocol isn't in our supported protocols we do not reply to the pong and remove this node since we don't support its version
	if (!SupportedProtocolVersions.Contains(ProtocolVersion))
	{
		RemoveKnownNode(NodeInfo.NodeId);
		return;
	}

	// Set this node protocol to our agreed protocol
	NodeInfo.ProtocolVersion = ProtocolVersion;

	// Send the pong
	FUdpMessageSegment::FHeader Header;
	{
		// Reply to the ping using the agreed protocol
		Header.ProtocolVersion = ProtocolVersion;
		Header.RecipientNodeId = NodeInfo.NodeId;
		Header.SenderNodeId = LocalNodeId;
		Header.SegmentType = EUdpMessageSegments::Pong;
	}

	TSharedRef<FArrayWriter, ESPMode::ThreadSafe> Writer = MakeShared<FArrayWriter, ESPMode::ThreadSafe>();
	{
		*Writer << Header;
		*Writer << LocalNodeId;
	}

	if (!SocketSender->Send(Writer, NodeInfo.Endpoint))
	{
		UE_LOGF(LogUdpMessaging, Warning, "FUdpMessageProcessor::ProcessPingSegment failed to send ping segment %ls.", *NodeInfo.Endpoint.ToString());
	}
}


void FUdpMessageProcessor::ProcessPongSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo)
{
	FGuid RemoteNodeId;
	*Segment.Data << RemoteNodeId;

	ensure(RemoteNodeId == NodeInfo.NodeId);
}

void FUdpMessageProcessor::ProcessRetransmitSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo)
{
	SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_ProcessRetransmitSegment);

	FUdpMessageSegment::FRetransmitChunk RetransmitChunk;
	int32 TargetMessageId = RetransmitChunk.GetMessageId(*Segment.Data);
	if (NodeInfo.Segmenters.IsEmpty() || !NodeInfo.Segmenters.Contains(TargetMessageId))
	{
		// Ignore this message because we have no segmenters to retransmit.
		return;
	}

	RetransmitChunk.Serialize(*Segment.Data, NodeInfo.ProtocolVersion);

	TSharedPtr<FUdpMessageSegmenter> Segmenter = NodeInfo.Segmenters.FindRef(RetransmitChunk.MessageId);

	UE_LOGF(LogUdpMessaging, Verbose, "Received retransmit for %d from %ls", RetransmitChunk.MessageId, *NodeInfo.NodeId.ToString());

	if (Segmenter.IsValid())
	{
		Segmenter->MarkForRetransmission(RetransmitChunk.Segments);
	}
	else
	{
		UE_LOGF(LogUdpMessaging, Verbose, "No such segmenter for message %d", RetransmitChunk.MessageId);
	}
}


void FUdpMessageProcessor::ProcessTimeoutSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo)
{
	SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_ProcessTimeoutSegment);

	if (NodeInfo.Segmenters.IsEmpty())
	{
		// Ignore this message because we have no segmenters to retransmit.
		return;
	}
	FUdpMessageSegment::FTimeoutChunk TimeoutChunk;
	TimeoutChunk.Serialize(*Segment.Data, NodeInfo.ProtocolVersion);

	TSharedPtr<FUdpMessageSegmenter> Segmenter = NodeInfo.Segmenters.FindRef(TimeoutChunk.MessageId);

	if (Segmenter.IsValid())
	{
		Segmenter->MarkForRetransmission();
	}
}

void FUdpMessageProcessor::ProcessUnknownSegment(FInboundSegment& Segment, FNodeInfo& EndpointInfo, uint8 SegmentType)
{
	UE_LOGF(LogUdpMessaging, Verbose, "Received unknown segment type '%i' from %ls", SegmentType, *Segment.Sender.ToText().ToString());
}

bool FUdpMessageProcessor::LookupAndCacheMessageType(TSharedPtr<FUdpReassembledMessage, ESPMode::ThreadSafe>& ReassembledMessage)
{
	if (ReassembledMessage->GetMessageTypeInfo().IsValid())
	{
		return true;
	}

	if (!ReassembledMessage->HasFirstSegment())
	{
		return false;
	}

	FNameOrFTopLevel AssetPath = FUdpDeserializedMessage::PeekMessageTypeInfoName(*ReassembledMessage);

	TRACE_UDPMESSAGING_MESSAGE_TYPEINFO(
		ReassembledMessage->GetTraceMetadata().SenderShortId,
		ReassembledMessage->GetTraceMetadata().RecipientShortId,
		ReassembledMessage->GetTraceMetadata().MessageId,
		AssetPath
	);

	FString PathAsString = Visit([](auto&& Path) -> FString { return Path.ToString(); }, AssetPath);
	if (TWeakObjectPtr<UScriptStruct>* TypeInfo = CachedTypeInfoMap.Find(PathAsString))
	{
		ReassembledMessage->SetMessageTypeInfo(*TypeInfo);
		return true;
	}
	else if (!UE::IsSavingPackage() && !IsGarbageCollecting())
	{
		// Otherwise we have to look up the object by calling FindObjectSafe.  This can fail in GC and package save.
		// Thus we only do this in not saving / GC cases.
		TWeakObjectPtr<UScriptStruct> Obj = FUdpDeserializedMessage::ResolvePath(AssetPath);
		CachedTypeInfoMap.Add(PathAsString, Obj);
		ReassembledMessage->SetMessageTypeInfo(MoveTemp(Obj));
		return true;
	}

	return false;
}

void FUdpMessageProcessor::TryDeliverMessage(const TSharedPtr<FUdpReassembledMessage>& ReassembledMessage, FNodeInfo& NodeInfo)
{
	// Do not deliver message while saving or garbage collecting since those deliveries will fail anyway...
	if (ReassembledMessage->GetMessageTypeInfo() == nullptr && (UE::IsSavingPackage() || IsGarbageCollecting()))
	{
		UE_LOGF(LogUdpMessaging, Verbose, "Skipping delivery of %ls", *ReassembledMessage->Describe());
		return;
	}

	TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
		FPlatformTime::Cycles64(),
		NodeInfo.NodeTraceId,
		LocalNodeTraceId,
		ReassembledMessage->GetTraceMetadata().MessageId,
		UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::PendingDelivery
	);

	if (NodeInfo.NodeId.IsValid())
	{
		MessageReassembledDelegate.ExecuteIfBound(ReassembledMessage, nullptr, NodeInfo.NodeId);
	}
	else
	{
		UE_LOGF(LogUdpMessaging, Display, "Unable to deliver %ls due to invalid NodeId", *ReassembledMessage->Describe());

		TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
			FPlatformTime::Cycles64(),
			ReassembledMessage->GetTraceMetadata().SenderShortId,
			ReassembledMessage->GetTraceMetadata().RecipientShortId,
			ReassembledMessage->GetTraceMetadata().MessageId,
			UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::Discarded
		);
	}

	// Mark the message pending delivery but do not remove it from the list yet, this is to prevent the double delivery of reliable message.
	ReassembledMessage->MarkPendingDelivery();
}


void FUdpMessageProcessor::RemoveKnownNode(const FGuid& NodeId)
{
	NodeLostDelegate.ExecuteIfBound(NodeId);
	KnownNodes.Remove(NodeId);
}

bool HasTimedOut(const FUdpMessagingTime& ReceivedTime, const FUdpMessagingTime& CurrentTime)
{
	const float TimeoutInSeconds = GetDefault<UUdpMessagingSettings>()->ConnectionTimeoutPeriod;
	FTimespan TimeoutInterval = FTimespan::FromSeconds(TimeoutInSeconds);
	return (ReceivedTime + TimeoutInterval) < CurrentTime;
}

void FUdpMessageProcessor::RemoveDeadNodes()
{
	// Remove dead nodes
	FTimespan DeadHelloTimespan = DeadHelloIntervals * Beacon->GetBeaconInterval();
	if (DeadHelloTimespan.IsZero())
	{
		// We never expire connections.
		return;
	}

	for (auto It = KnownNodes.CreateIterator(); It; ++It)
	{
		FGuid& NodeId = It->Key;
		FNodeInfo& NodeInfo = It->Value;

		if ((NodeId.IsValid()) && HasTimedOut(NodeInfo.LastSegmentReceivedTime, FUdpMessagingTime::Now()))
		{
			UE_LOGF(LogUdpMessaging, Display, "FUdpMessageProcessor::UpdateKnownNodes: Removing Node %ls (%ls)", *NodeInfo.NodeId.ToString(), *NodeInfo.Endpoint.ToString());
			NodeLostDelegate.ExecuteIfBound(NodeId);
			It.RemoveCurrent();
		}
	}
}

bool FUdpMessageProcessor::MoreToSend()
{
	for (auto& KnownNodePair : KnownNodes)
	{
		if (KnownNodePair.Value.CanSendSegments() && KnownNodePair.Value.HasSegmenterThatCanSend(FUdpMessagingTime::Now()))
		{
			return true;
		}
	}
	return false;
}

void FUdpMessageProcessor::HandleSocketError(const TCHAR* ErrorReference, const FNodeInfo& NodeInfo) const
{
	UE_LOGF(LogUdpMessaging, Error, "%ls, Socket error detected when communicating with %ls, Banning communication to that endpoint.", ErrorReference, *NodeInfo.Endpoint.ToString());
	ErrorSendingToEndpointDelegate.Execute(NodeInfo.NodeId, NodeInfo.Endpoint);
}

bool FUdpMessageProcessor::CanSendKnownNodesToKnownNodes() const
{
	return bShareKnownNodes && bAddedNewKnownNodes;
}

void FUdpMessageProcessor::SendKnownNodesToKnownNodes()
{
	FUdpMessageSegment::FHeader Header;
	{
		Header.SenderNodeId = LocalNodeId;
		Header.ProtocolVersion = UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION;
		Header.SegmentType = EUdpMessageSegments::Mesh;
	};

	// First pass: gather everything to be sent.
	TArray<FIPv4Endpoint> Endpoints, OutEndpoints;
	TArray<FGuid> EndpointIds, OutIds;
	for (const TPair<FGuid, FNodeInfo>& NodePair : KnownNodes)
	{
		EndpointIds.Add(NodePair.Key);
		Endpoints.Add(NodePair.Value.Endpoint);
	}

	// Second pass: send to everyone (segment size can vary per peer node).
	for (const TPair<FGuid, FNodeInfo>& NodePair : KnownNodes)
	{
		// We use TArrayView to slice parts of the array for sending (potentially multiple segments).
		TArrayView<FIPv4Endpoint> KnownEndpointsView(Endpoints);
		TArrayView<FGuid> KnownIdView(EndpointIds);

		const uint16 MeshSegmentFixedBytes = FUdpMessageSegment::FHeader::SerializedBytes
			+ sizeof(decltype(OutEndpoints)::SizeType)
			+ sizeof(decltype(OutIds)::SizeType);

		if (!ensure(MeshSegmentFixedBytes < NodePair.Value.MaxSegmentSize))
		{
			continue;
		}

		const uint16 MaxMeshPayload = NodePair.Value.MaxSegmentSize - MeshSegmentFixedBytes;
		const int32 NumEndpointsCanSend = MaxMeshPayload / (sizeof(FIPv4Endpoint) + sizeof(FGuid)) - 1;

		if (!ensure(NumEndpointsCanSend > 0))
		{
			continue;
		}

		int32 Index = 0;
		while (Index < KnownEndpointsView.Num())
		{
			const int32 NextBlockItemCount = FMath::Min(NumEndpointsCanSend, KnownEndpointsView.Num() - Index);
			TArrayView<FIPv4Endpoint> SlicedEndpointView = KnownEndpointsView.Slice(Index, NextBlockItemCount);
			TArrayView<FGuid>	      SlicedIdView = KnownIdView.Slice(Index, NextBlockItemCount);

			OutEndpoints = SlicedEndpointView;
			OutIds = SlicedIdView;

			UE_LOGF(LogUdpMessaging, VeryVerbose, "FUdpMessageProcessor::SendKnownNodesToKnownNodes Sending updated known nodes.");

			TSharedRef<FArrayWriter, ESPMode::ThreadSafe> Writer = MakeShared<FArrayWriter, ESPMode::ThreadSafe>();
			{
				*Writer << Header;
				*Writer << OutEndpoints;
				*Writer << OutIds;
			}

			const FIPv4Endpoint& Endpoint = NodePair.Value.Endpoint;
			if (!SocketSender->Send(Writer, Endpoint))
			{
				UE_LOGF(LogUdpMessaging, Warning, "FUdpMessageProcessor::SendKnownNodesToKnownNodes failed to share endpoint information to %ls.", *Endpoint.ToString());
			}

			Index += NumEndpointsCanSend;
		}
	}

	bAddedNewKnownNodes = false;
}

void FUdpMessageProcessor::ProcessMeshSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo)
{
	TArray<FGuid> Ids;
	TArray<FIPv4Endpoint> Endpoints;
	*Segment.Data << Endpoints;
	*Segment.Data << Ids;

	check(Ids.Num() == Endpoints.Num());
	for (int32 Index = 0; Index < Ids.Num(); Index ++)
	{
		if (LocalNodeId != Ids[Index] && !KnownNodes.Find(Ids[Index]))
		{
			AddStaticEndpoint(Endpoints[Index]);
		}
	}
}

void FUdpMessageProcessor::UpdateKnownNodes()
{
	SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_UpdateKnownNodes);

	RemoveDeadNodes();

	UpdateNodesPerVersion();
	Beacon->SetEndpointCount(KnownNodes.Num() + 1);

	if (CanSendKnownNodesToKnownNodes())
	{
		SendKnownNodesToKnownNodes();
	}

	float OutboundMessagesMemory = 0.0f;
	float ReassemblersMemory = 0.0f;
	constexpr uint32 LogFrequencySeconds = 5 * 60;
	bool bCanLogDebugInfo = FPlatformTime::Seconds() > LastReassemblersLogTime + LogFrequencySeconds;

	bool bSuccess = true;
	for (auto& KnownNodePair : KnownNodes)
	{
		int32 NodeByteSent = UpdateSegmenters(KnownNodePair.Value);
		// if NodByteSent is negative, there is a socket error, continuing is useless
		bSuccess = NodeByteSent >= 0;
		if (!bSuccess || UE::Private::MessageProcessor::ShouldErrorOnConnection(KnownNodePair.Value.Endpoint))
		{
			bSuccess = false; // To ensure we trigger the delegate on a forced error.
			HandleSocketError(TEXT("UpdateSegmenters"), KnownNodePair.Value);
			break;
		}

		bSuccess = UpdateReassemblers(KnownNodePair.Value);
		// if there is a socket error, continuing is useless
		if (!bSuccess)
		{
			HandleSocketError(TEXT("UpdateReassemblers"), KnownNodePair.Value);
			break;
		}

		if (MemoryLoggingThresholdMB >= 0 && bCanLogDebugInfo)
		{
			uint32 ReassembledTotalBytes = 0;
			for (const TPair<int32, TSharedPtr<FUdpReassembledMessage>>& Pair : KnownNodePair.Value.ReassembledMessages)
			{
				if (Pair.Value)
				{
					ReassembledTotalBytes += sizeof(FUdpReassembledMessage) + Pair.Value->GetReceivedBytes();
				}
			}

			OutboundMessagesMemory += ReassembledTotalBytes / 1'000'000.f;
			ReassemblersMemory += KnownNodePair.Value.Statistics.BytesInflight / 1'000'000.f;
		}
	}

	// Only start logging once we accumulate more memory than what was specified in the UDP Messaging config.
	if (MemoryLoggingThresholdMB >= 0 && OutboundMessagesMemory + ReassemblersMemory > MemoryLoggingThresholdMB && bCanLogDebugInfo)
	{
		UE_LOGF(LogUdpMessaging, Log, "(Debug) MessageBus Memory Usage: Reassemblers: %.2f MB, Outbound messages: %.2f MB", ReassemblersMemory, OutboundMessagesMemory);
		LastReassemblersLogTime = FPlatformTime::Seconds();
	}

	// if we had socket error, fire up the error delegate
	if (!bSuccess || Beacon->HasSocketError())
	{
		bStopping = true;
		ErrorDelegate.ExecuteIfBound();
	}
}

TSharedPtr<FUdpMessageSegmenter>* FUdpMessageProcessor::GetSegmenter(FNodeInfo& NodeInfo, int32 MessageId)
{
	TSharedPtr<FUdpMessageSegmenter>* FoundSegmenter = NodeInfo.Segmenters.Find(MessageId);
	if (!FoundSegmenter)
	{
		// It's possible that a segmenter was fully ack and removed from the segmenter list.
		return nullptr;
	}

	TSharedPtr<FUdpMessageSegmenter>& Segmenter = *FoundSegmenter;
	if (!Segmenter->IsInitialized())
	{
		// This is essentially polling on the async serialization to complete.
		if (Segmenter->TryInitialize())
		{
#if UDPMESSAGINGTRACE_ENABLED
			Segmenter->EmitPostInitializeTraceEvents(LocalNodeTraceId, NodeInfo.NodeTraceId, MessageId);
#endif
		}
	}

	if (Segmenter->IsInvalid())
	{
		UE_LOGF(LogUdpMessaging, Warning, "FUdpMessageProcessor::GetInitializedSegments segmenter is invalid. Removing from send queue.");
		NodeInfo.Segmenters.Remove(MessageId);
		TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
			FPlatformTime::Cycles64(),
			LocalNodeTraceId,
			NodeInfo.NodeTraceId,
			MessageId,
			UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::Discarded
		);
		return nullptr;
	}

	return FoundSegmenter;
}

bool FUdpMessageProcessor::FillDataChunk(FUdpMessageSegment::FDataChunk& OutDataChunk, FNodeInfo& NodeInfo, TSharedPtr<FUdpMessageSegmenter>& Segmenter, int32 MessageId)
{
	// Track the segments we sent as we'll update the segmenter to keep track
	uint32 OutSegmentNumber;
	if (!Segmenter->GetNextPendingSegment(OutDataChunk.Data, OutSegmentNumber))
	{
		return false;
	}

	OutDataChunk.MessageId = MessageId;
	OutDataChunk.MessageSize = Segmenter->GetMessageSize();
	OutDataChunk.MessageFlags = Segmenter->GetMessageFlags();
	OutDataChunk.SegmentNumber = OutSegmentNumber;
	OutDataChunk.SegmentOffset = OutSegmentNumber * static_cast<uint64>(Segmenter->GetSegmentPayloadSize());
	OutDataChunk.TotalSegments = Segmenter->GetSegmentCount();
	OutDataChunk.Sequence = 0; // This should be kept 0 for legacy support.

	return true;
}

int32 FUdpMessageProcessor::ClampMaxSegments(int32 MaxSegments, const TSharedPtr<FUdpMessageSegmenter>& Segmenter, const FNodeInfo& NodeInfo) const
{
	const int32 SegmentOffloadPayloadSize = SocketSender->GetSegmentOffloadSize();
	if (SegmentOffloadPayloadSize > 0
		&& NodeInfo.MaxSegmentSize >= SegmentOffloadPayloadSize
		&& Segmenter->GetTotalSegmentSize() == SegmentOffloadPayloadSize)
	{
		// Hardware acceleration frequently limits the pre-segmented size to 64K.
		const int32 SenderMaxSubmitSize = UINT16_MAX;
		const int32 SenderMaxSubmitSegments = SenderMaxSubmitSize / SegmentOffloadPayloadSize;
		MaxSegments = FMath::Min(MaxSegments, SenderMaxSubmitSegments);

		// "UDP_SEGMENT [...] The segment size must be chosen such that at most 64 datagrams are sent in a single call" - man udp(7)
		const int32 SenderMaxGeneratedSegments = 64;
		MaxSegments = FMath::Min(MaxSegments, SenderMaxGeneratedSegments);

		const int32 CvarMaxGeneratedSegments = CVarUdpMessagingMaxOffloadSegments.GetValueOnAnyThread();
		MaxSegments = FMath::Clamp(MaxSegments, 1, CvarMaxGeneratedSegments);
	}
	else
	{
		// Segmentation offload not supported; only send a single segment at a time.
		MaxSegments = 1;
	}

	return MaxSegments;
}

FSentSegmentInfo FUdpMessageProcessor::SendNextSegmentsForMessageId(
	FNodeInfo& NodeInfo,
	FUdpMessageSegment::FHeader& Header,
	int32 MessageId,
	int32 MaxSegments /* = 1 */
)
{
	TSharedPtr<FUdpMessageSegmenter>* FoundSegmenter = GetSegmenter(NodeInfo, MessageId);
	if (!FoundSegmenter)
	{
		// It's possible that a segmenter was fully ack and removed from the segmenter list.
		return FSentSegmentInfo(MessageId);
	}

	TSharedPtr<FUdpMessageSegmenter>& Segmenter = *FoundSegmenter;

	FSentSegmentInfo SentInfo(MessageId);
	TArray<FSentData, TInlineAllocator<64>> SentDatas;
	if (Segmenter->IsInitialized() && !Segmenter->IsSendingComplete())
	{
		if (Header.ProtocolVersion != Segmenter->GetProtocolVersion())
		{
			UE_LOGF(LogUdpMessaging, Error,
				"FUdpMessageProcessor::SendNextSegmentsForMessageId: Node %ls initially asked for protocol version %d but changed to %d. Cannot deliver message %d",
				*NodeInfo.Endpoint.ToString(), Segmenter->GetProtocolVersion(), Header.ProtocolVersion, MessageId);
			TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
				FPlatformTime::Cycles64(),
				LocalNodeTraceId,
				NodeInfo.NodeTraceId,
				MessageId,
				UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::Discarded
			);
			NodeInfo.Segmenters.Remove(MessageId);
			return FSentSegmentInfo(MessageId);
		}

		const bool bIsReliable = EnumHasAnyFlags(Segmenter->GetMessageFlags(), EMessageFlags::Reliable);

		MaxSegments = ClampMaxSegments(MaxSegments, Segmenter, NodeInfo);

		TSharedRef<FArrayWriter, ESPMode::ThreadSafe> Writer = MakeShared<FArrayWriter, ESPMode::ThreadSafe>();
		Writer->Reserve(NodeInfo.MaxSegmentSize * MaxSegments);

		FUdpMessageSegment::FDataChunk DataChunk;

		for (int32 SendIdx = 0; SendIdx < MaxSegments; ++SendIdx)
		{
			if (Segmenter->IsSendingComplete())
			{
				break;
			}

			if (!FillDataChunk(DataChunk, NodeInfo, Segmenter, MessageId))
			{
				UE_LOGF(LogUdpMessaging, Error,
					"FUdpMessageProcessor::SendNextSegmentsForMessageId: FillDataChunk failed");
				break;
			}

			const int64 WriteStartPos = Writer->Tell();
			*Writer << Header;
			DataChunk.Serialize(*Writer, Header.ProtocolVersion);
			const int64 WriteEndPos = Writer->Tell();
			const int64 BytesWritten = WriteEndPos - WriteStartPos;

			UE_LOG(LogUdpMessaging, VeryVerbose, TEXT("FUdpMessageProcessor::UpdateSegmenters: Sending msg %d as segment %d/%d of %" INT64_FMT " bytes to %s"),
				MessageId,
				DataChunk.SegmentNumber + 1,
				DataChunk.TotalSegments,
				Segmenter->GetMessageSize(),
				*NodeInfo.NodeId.ToString());

			// This is optimistic; we'll undo it later if the batched send fails.
			Segmenter->MarkAsSent(DataChunk.SegmentNumber);

			SentDatas.Emplace(
				MessageId,
				DataChunk.SegmentNumber,
				++NodeInfo.SequenceId,
				bIsReliable,
				FUdpMessagingTime::Now()
			);
		}

		const bool bAllowSegmentation_true = true;
		if (SocketSender->Send(Writer, NodeInfo.Endpoint, bAllowSegmentation_true))
		{
			SentInfo.SequenceNumber = NodeInfo.SequenceId;
			SentInfo.BytesSent = Writer->Tell();
			SentInfo.SegmentsSent = SentDatas.Num();
			SentInfo.bIsReliable = bIsReliable;
			SentInfo.bRequiresRequeue = !Segmenter->IsSendingComplete() || SentInfo.bIsReliable;
			SentInfo.bFullySent = Segmenter->IsSendingComplete();
			SentInfo.bIsTraced = EnumHasAnyFlags(Segmenter->GetMessageFlags(), EMessageFlags::Traced);

			NodeInfo.Statistics.PacketsSent += SentDatas.Num();

			for (FSentData& SentData : SentDatas)
			{
				NodeInfo.AddInflight(SentData);
			}

			SendSegmenterStatsToListeners(MessageId, NodeInfo.NodeId, Segmenter);

			if (Segmenter->IsSendingComplete() && !bIsReliable)
			{
				// Not reliably sent so we don't need to wait for acks.
				TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
					FPlatformTime::Cycles64(),
					LocalNodeTraceId,
					NodeInfo.NodeTraceId,
					MessageId,
					UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::Complete
				);
				UE_LOGF(LogUdpMessaging, VeryVerbose, "FUdpMessageProcessor::UpdateSegmenters: Finished with message segmenter for %ls", *NodeInfo.NodeId.ToString());
				NodeInfo.Segmenters.Remove(MessageId);
			}
		}
		else
		{
			for (const FSentData& SentData : SentDatas)
			{
				// Roll back the sent state of any segments we failed to send.
				NodeInfo.RemoveInflight(SentData);
				Segmenter->MarkForRetransmission(SentData.SegmentNumber);
			}

			SentInfo.bSendSocketError = true;
			SentInfo.SegmentsSent = 0;
			return MoveTemp(SentInfo);
		}
	}
	else
	{
		const bool bIsReady = Segmenter->IsInitialized(); 
		if (bIsReady)
		{
			SentInfo.bIsTraced = EnumHasAnyFlags(Segmenter->GetMessageFlags(), EMessageFlags::Traced);
			SentInfo.bIsReliable = EnumHasAnyFlags(Segmenter->GetMessageFlags(), EMessageFlags::Reliable);
			SentInfo.bFullySent = Segmenter->IsSendingComplete();
		}
		SentInfo.bRequiresRequeue = true;
		SentInfo.bIsReady = bIsReady;
	}

	NodeInfo.Statistics.TotalBytesSent += SentInfo.BytesSent;
	NodeInfo.Statistics.IPv4AsString = NodeInfo.Endpoint.ToString();
	return MoveTemp(SentInfo);
}

TOptional<FSentSegmentInfoArray> FUdpMessageProcessor::DrainQueue(FNodeInfo& NodeInfo, FNodeInfo::FWorkQueue& WorkQueue, int64 DrainCount, const bool bDrainOverflow)
{
	FUdpMessageSegment::FHeader Header
	{
		NodeInfo.ProtocolVersion,		// Header.ProtocolVersion - Send data segment using the node protocol version
		NodeInfo.NodeId,				// Header.RecipientNodeId
		LocalNodeId,					// Header.SenderNodeId
		EUdpMessageSegments::Data		// Header.SegmentType
	};

	FSentSegmentInfoArray ProcessedMessages;

	if (bDrainOverflow)
	{
		// Check for acknowledged segments first.
		while (DrainCount > 0 && NodeInfo.CanSendSegments() && !NodeInfo.OverflowForPendingAck.IsEmpty())
		{
			const int32 MessageId = NodeInfo.OverflowForPendingAck.Pop();
			FSentSegmentInfo Info = SendNextSegmentsForMessageId(NodeInfo, Header, MessageId, DrainCount);
			if (Info.bSendSocketError)
			{
				return {};
			}
			ProcessedMessages.Emplace(MoveTemp(Info));
			DrainCount -= FMath::Max(1u, Info.SegmentsSent);
		}
	}

	// Process messages in the work queue.
	while (DrainCount > 0 && NodeInfo.CanSendSegments() && !WorkQueue.IsEmpty())
	{
		const int32 MessageId = WorkQueue.PopFrontValue();
		FSentSegmentInfo Info = SendNextSegmentsForMessageId(NodeInfo, Header, MessageId, DrainCount);
		if (Info.bSendSocketError)
		{
			return {};
		}
		ProcessedMessages.Emplace(MoveTemp(Info));
		DrainCount -= FMath::Max(1u, Info.SegmentsSent);
	}

	return ProcessedMessages;
}

int32 FUdpMessageProcessor::RequeueWork(FNodeInfo& NodeInfo, FNodeInfo::FWorkQueue& WorkQueue, const FSentSegmentInfoArray& ProcessedMessages)
{
	int32 BytesSent = 0;
	for (const FSentSegmentInfo& Info : ProcessedMessages)
	{
		if (Info.bRequiresRequeue && Info.bFullySent)
		{
			NodeInfo.OverflowForPendingAck.Add(Info.MessageId);
		}
		else if (Info.bRequiresRequeue)
		{
			WorkQueue.Add(Info.MessageId);
		}

		BytesSent += Info.BytesSent;
	}
	return BytesSent;	
}

bool FUdpMessageProcessor::ProcessQueue(FNodeInfo& NodeInfo, FNodeInfo::FWorkQueue& WorkQueue, int64 DrainLimit, int32& BytesSent, bool bShouldDrainOverflowQueue)
{
	int64 SendCapacity = NodeInfo.SendCapacity();
	int64 MaxSend = SendCapacity - DrainLimit;

	while (!WorkQueue.IsEmpty() && NodeInfo.CanSendSegments() && SendCapacity > DrainLimit)
	{
		if (TOptional<FSentSegmentInfoArray> ProcessedMessages = DrainQueue(NodeInfo, WorkQueue, MaxSend, bShouldDrainOverflowQueue))
		{
			bShouldDrainOverflowQueue = false;
			BytesSent += RequeueWork(NodeInfo, WorkQueue, *ProcessedMessages);
			SendCapacity = NodeInfo.SendCapacity();
			MaxSend = SendCapacity - DrainLimit;
		}
		else
		{
			// Socket Error
			return false;
		}
	}
	return true;
}

int32 FUdpMessageProcessor::UpdateSegmenters(FNodeInfo& NodeInfo)
{
	SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_UpdateSegmenters);

	const float ReliableQueuePriority = GetDefault<UUdpMessagingSettings>()->ReliableQueuePriority / 100.f;

	int32 BytesSent = 0;
	int64 SendCapacity = NodeInfo.SendCapacity();
	const int64 SendRoomForUnreliables = (1 - ReliableQueuePriority) * SendCapacity;
	int64 MaxSendForReliables = SendCapacity - SendRoomForUnreliables;

	// Prioritize the reliable queue first. The reliable queue always gets more bandwidth.  We drain the overflow queue first and only
	// on the first run as we don't want it to monopolize our time either.
	// 
	if (!ProcessQueue(NodeInfo, NodeInfo.ReliableWorkQueue, SendRoomForUnreliables, BytesSent, true))
	{
		// Socket error 
		return -1;
	}

	if (!ProcessQueue(NodeInfo, NodeInfo.UnreliableWorkQueue, 0, BytesSent))
	{
		// Socket error 
		return -1;
	}

	if (NodeInfo.SendCapacity() > 0)
	{
		// If there is still more room left go back to work on the reliable queue until we have nothing left or run out
		// of capacity.
		if (!ProcessQueue(NodeInfo, NodeInfo.ReliableWorkQueue, 0, BytesSent))
		{
			// Socket error 
			return -1;
		}
	}

	// RemoveLostSegments will clean-up the tracking map that we use to determine what is "inflight".
	// Old data will be expired and scheduled for resend.
	const uint32 LostSegmentCount = NodeInfo.ExpireSegments(FUdpMessagingTime::Now());
	NodeInfo.ComputeWindowSize(0, LostSegmentCount);

	NodeInfo.ExpireRecentDeliveries();

	NodeInfo.Statistics.PacketsInFlight = NodeInfo.InflightSegments.Num();
	NodeInfo.Statistics.BytesInflight = NodeInfo.InflightSegments.Num() * NodeInfo.MaxSegmentSize;

	return BytesSent;
}

const FTimespan FUdpMessageProcessor::StaleReassemblyInterval = FTimespan::FromSeconds(30);

bool FUdpMessageProcessor::UpdateReassemblers(FNodeInfo& NodeInfo)
{
	SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_UpdateReassemblers);

	FUdpMessageSegment::FHeader Header
	{
		FMath::Max(NodeInfo.ProtocolVersion, (uint8)11),	// Header.ProtocolVersion, AcknowledgeSegments are version 11 and onward segment
		NodeInfo.NodeId,									// Header.RecipientNodeId
		LocalNodeId,										// Header.SenderNodeId
		EUdpMessageSegments::AcknowledgeSegments			// Header.SegmentType
	};

	const uint16 MaxAckSegmentPayload = NodeInfo.MaxSegmentSize
		- FUdpMessageSegment::FHeader::SerializedBytes
		- FUdpMessageSegment::FAcknowledgeSegmentsChunk::FixedSerializedBytes;

	const int32 MaxAckNum = (MaxAckSegmentPayload / sizeof(uint32));
	if (!ensure(MaxAckNum > 0))
	{
		return true;
	}

	const FUdpMessagingTime StaleLastSegmentTime = FUdpMessagingTime::Now() - StaleReassemblyInterval;

	for (TMap<int32, TSharedPtr<FUdpReassembledMessage, ESPMode::ThreadSafe>>::TIterator It(NodeInfo.ReassembledMessages); It; ++It)
	{
		const int32 MessageId = It.Key();
		TSharedPtr<FUdpReassembledMessage, ESPMode::ThreadSafe>& ReassembledMessagePtr = It.Value();
		// Believe it or not, TSharedPtr operator-> otherwise makes a big showing here.
		FUdpReassembledMessage* ReassembledMessage = It.Value().Get();

		if (ReassembledMessage->IsDelivered())
		{
			// Stop tracking the reassembler, but make a note to suppress double delivery of late retransmits.
			It.RemoveCurrent();
			NodeInfo.RecentlyDeliveredMessages.Add(MessageId);
			NodeInfo.DeliveryExpiry.Emplace(FUdpMessagingTime::Now(), MessageId);
			continue;
		}
		else if (ReassembledMessage->IsPendingDelivery())
		{
			// Leave it in the list, but skip further processing.
			continue;
		}

		SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_UpdateReassemblers_Message);

		const FUdpMessagingTime MaxAckDelay = FUdpMessagingTime::Now() - FTimespan::FromMilliseconds(5);
		const bool bShouldAck = (ReassembledMessage->GetLastAckTime() < MaxAckDelay) || (ReassembledMessage->NumPendingAcknowledgements() >= MaxAckNum);

		int BytesSent = 0;
		while (bShouldAck && ReassembledMessage->NumPendingAcknowledgements() > 0)
		{
			SCOPED_MESSAGING_TRACE(FUdpMessageProcessor_UpdateReassemblers_Acks);
			TArray<uint32> PendingAcknowledgments = ReassembledMessage->TakePendingAcknowledgments(MaxAckNum, FUdpMessagingTime::Now());
			const int32 AckCount = PendingAcknowledgments.Num();

			FUdpMessageSegment::FAcknowledgeSegmentsChunk AcknowledgeChunk(MessageId, MoveTemp(PendingAcknowledgments)/*Segments*/);
			TSharedRef<FArrayWriter, ESPMode::ThreadSafe> Writer = MakeShared<FArrayWriter, ESPMode::ThreadSafe>();
			{
				*Writer << Header;
				AcknowledgeChunk.Serialize(*Writer, Header.ProtocolVersion);
			}
			BytesSent += Writer->Num();

			UE_LOGF(LogUdpMessaging, VeryVerbose, "FUdpMessageProcessor::UpdateReassemblers: Sending EUdpMessageSegments::AcknowledgeSegments for %d segments for message %d from %ls",
				AckCount, MessageId, *ReassembledMessage->Describe());

			const bool bAllowSegmentation_false = false;
			const bool bHighPriority_true = true;
			if (!SocketSender->Send(Writer, NodeInfo.Endpoint, bAllowSegmentation_false, bHighPriority_true))
			{
				UE_LOGF(LogUdpMessaging, Error, "FUdpMessageProcessor::UpdateReassemblers: error sending EUdpMessageSegments::AcknowledgeSegments from %ls", *NodeInfo.NodeId.ToString());
				return false;
			}
		}

		// Try to deliver completed message that couldn't be delivered the first time around
		if (ReassembledMessage->IsComplete() && !(ReassembledMessage->IsPendingDelivery() || ReassembledMessage->IsDelivered()))
		{
			TryDeliverMessage(ReassembledMessagePtr, NodeInfo);
		}

		// Remove stale reassembled message if they aren't reliable or are marked delivered
		if (ReassembledMessage->GetLastSegmentTime() <= StaleLastSegmentTime &&
			(!EnumHasAnyFlags(ReassembledMessage->GetFlags(), EMessageFlags::Reliable)))
		{
			if (!ReassembledMessage->IsDelivered() && !ReassembledMessage->IsPendingDelivery())
			{
				const int ReceivedSegments = ReassembledMessage->GetTotalSegmentsCount() - ReassembledMessage->GetPendingSegmentsCount();
				UE_LOGF(LogUdpMessaging, Warning, "FUdpMessageProcessor::UpdateReassemblers Discarding %d/%d of stale message segments from %ls",
					ReceivedSegments, ReassembledMessage->GetTotalSegmentsCount(), *ReassembledMessage->Describe());
			}

			if (!ReassembledMessage->IsPendingDelivery())
			{
				TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
					FPlatformTime::Cycles64(),
					ReassembledMessagePtr->GetTraceMetadata().SenderShortId,
					ReassembledMessagePtr->GetTraceMetadata().RecipientShortId,
					ReassembledMessagePtr->GetTraceMetadata().MessageId,
					UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::Discarded
				);
				PendingTypeLookup.Remove(ReassembledMessagePtr);
				It.RemoveCurrent();
			}
		}
	}
	return true;
}


void FUdpMessageProcessor::UpdateNodesPerVersion()
{
	FScopeLock NodeVersionLock(&NodeVersionCS);
	NodeVersions.Empty();
	for (auto& NodePair : KnownNodes)
	{
		NodeVersions.Add(NodePair.Key, NodePair.Value.ProtocolVersion);
	}
}
