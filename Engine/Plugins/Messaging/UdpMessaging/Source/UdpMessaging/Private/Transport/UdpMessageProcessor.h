// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/MpscQueue.h"
#include "Containers/RingBuffer.h"

#include "HAL/Platform.h"
#include "ModuleDescriptor.h"

#include "HAL/Runnable.h"

#include "INetworkMessagingExtension.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/SingleThreadRunnable.h"

#include "UObject/TopLevelAssetPath.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "Shared/UdpMessagingSettings.h"
#include "Shared/UdpMessagingTime.h"
#include "UdpMessageSegmenter.h"
#include "UdpMessagingPrivate.h"
#include "Trace/UdpMessagingTrace.h"

#include <atomic>


class IMessageContext;
namespace FUdpMessageSegment { struct FDataChunk; }
namespace FUdpMessageSegment { struct FHeader; }
namespace UE::UdpMessaging { class FSocketSender; }

class FArrayReader;
class FEvent;
class FRunnableThread;
class FSocket;
class FUdpMessageBeacon;
class FUdpMessageSegmenter;
class FUdpReassembledMessage;
class FUdpSerializedMessage;
class IMessageAttachment;
enum class EUdpMessageFormat : uint8;
enum class EUdpMessageSegments : uint8;
using FUdpReassembledMessageSharedPtr = TSharedPtr<FUdpReassembledMessage, ESPMode::ThreadSafe>;


/** Holds the current statistics for a given message segmenter. */
struct FUdpSegmenterStats
{
	FGuid  NodeId;
	int32  MessageId     = 0;
	uint32 SegmentsSent  = 0;
	uint32 SegmentsAcked = 0;
	uint32 TotalSegments = 0;
};

/** Information about what was sent over the wire.  */
struct FSentData
{
	int32 MessageId = 0;
	uint32 SegmentNumber = 0;

	uint64 SequenceNumber = 0;
	bool bIsReliable = false;

	FUdpMessagingTime TimeSent;
};

/** Lightweight tracking of segment transmit times to expedite maintenance of send window. */
struct FSegmentSendExpiry
{
	FUdpMessagingTime TimeSent;
	int32 MessageId = 0;
	uint32 SegmentNumber = 0;
};

/** Lightweight tracking of message delivery times to prevent double delivery. */
struct FMessageDeliverExpiry
{
	FUdpMessagingTime TimeDelivered;
	int32 MessageId = 0;
};

/** Returned by the segment processor. It provides details on what was sent and how it was sent. */
struct FSentSegmentInfo
{
	FSentSegmentInfo() = delete;
	FSentSegmentInfo(int32 InMessageId)
		: MessageId(InMessageId)
	{
	}

	int32 MessageId;
	uint64 SequenceNumber = 0;
	uint32 BytesSent = 0;
	uint32 SegmentsSent = 0;

	bool bIsReliable = false;
	bool bRequiresRequeue = false;
	bool bSendSocketError = false;
	bool bFullySent = false;
	bool bIsTraced = false;
	bool bIsReady = false;

	bool WasSent() const
	{
		return BytesSent > 0;
	}
};

using FSentSegmentInfoArray = TArray<FSentSegmentInfo, TInlineAllocator<64>>;

namespace UE::Private::MessageProcessor
{
/**
 * Global delegate for handling segmenter (aka sent data) updates.
 */
FOnOutboundTransferDataUpdated &OnSegmenterUpdated();

/*
 * Global delegate for handling reassembler (aka received data) updates.
 */
FOnInboundTransferDataUpdated &OnReassemblerUpdated();

}

/**
 * Heap allocator to avoid range checks for performance reasons.
 */
class FUdpMessagingRangeChecklessHeapAllocator : public FHeapAllocator
{
public:
	/** Don't want to lose performance on range checks in performance-critical code. */
	enum { RequireRangeCheck = false };
};

/**
 * Implements a message processor for UDP messages.
 */
class FUdpMessageProcessor
	: public FRunnable
	, private FSingleThreadRunnable
{
	/** Structure for known remote endpoints. */
	struct FNodeInfo
	{
		/** Holds the node's IP endpoint. */
		FIPv4Endpoint Endpoint;

		/** Holds the time at which we last received any segment from this node. */
		FUdpMessagingTime LastSegmentReceivedTime;

		/** Holds the endpoint's node identifier. */
		FGuid NodeId;

		/** Holds the protocol version this node is communicating with */
		uint8 ProtocolVersion;

		/** The maximum datagram payload (UDP SDU) this node is capable of receiving. Clamped to USO size if enabled. */
		uint16 MaxSegmentSize;

		/** Holds the collection of reassembled messages. */
		TMap<int32, TSharedPtr<FUdpReassembledMessage>> ReassembledMessages;

		/** Holds the collection of message segmenters. */
		TMap<int32, TSharedPtr<FUdpMessageSegmenter>> Segmenters;

		using FWorkQueue = TRingBuffer<int32, FUdpMessagingRangeChecklessHeapAllocator>;

		/** Holds of queue of MessageIds to send for reliable messages. They are processed in round-robin fashion. */
		FWorkQueue ReliableWorkQueue;

		/** Holds of queue of MessageIds to send for unreliable messages. They are processed in round-robin fashion. */
		FWorkQueue UnreliableWorkQueue;

		/* Holds the list of messages that are still pending acknowledgement. */
		TArray<int32> OverflowForPendingAck;

		/** A map from sequence id to information about what was sent. The size of this map is the size of our sending window. */
		TMap<uint64, FSentData> InflightSegments;

		/** Send timings for outgoing reliable segments. */
		TRingBuffer<FSegmentSendExpiry> ReliableExpiry;

		/** Send timings for outgoing unreliable segments. */
		TRingBuffer<FSegmentSendExpiry> UnreliableExpiry;

		/** Messages delivered recently enough to track and suppress double delivery. */
		TSet<int32> RecentlyDeliveredMessages;

		/** Delivery times for recently delivered messages. */
		TRingBuffer<FMessageDeliverExpiry> DeliveryExpiry;

		/** Unique sequence id for every packet sent. */
		uint64 SequenceId = 0;

		/** Maximum number of segments we can have in flight. Start with a middle of the road value and adjust based on network performance. */
		uint64 WindowSize = 1024;

		/** Maximum window size that can be set. */
		uint64 MaxWindowSize = 2048;

		/** A smoothed calculated RTT that is effectively the weighted average. */
		double SmoothedRTT = -1.0;

		/** Measured variance of the RTT to be used to calculate a retransmission timeout. */
		double RTTVariance = 0.0;

		/**
		 * Set a high retransmission timeout on init until we start measuring speed and overall jitter.
		 * The default is 1 second. In other words, if we don't get an ack within 1 second, we assume it was
		 * lost and thus should be marked for retransmission. This value will adjust lower as we compute
		 * new data especially on a fast network.
		 */
		double RetransmissionTimeout = 1.0f;

		/** Various transport statistics for this endpoint */
		FMessageTransportStatistics Statistics;

#if UDPMESSAGINGTRACE_ENABLED
		/** ID used for trace events. */
		uint16 NodeTraceId;
#endif

		/** Default constructor. */
		FNodeInfo();

		/**
		 * Remap a MessageId and SegmentId pair into a 64 bit number using Szudzik pairing calculation.
		 */
		uint64 Remap(uint32 MessageId, uint32 SegmentId)
		{
			// http://szudzik.com/ElegantPairing.pdf
			// a >= b ? a * a + a + b : a + b * b
			//
			if (MessageId >= SegmentId)
			{
				return (uint64)MessageId * MessageId + MessageId + SegmentId;
			}
			else
			{
				return (uint64)SegmentId * SegmentId + MessageId;
			}
		}

		/**
		 * Update the segmenter with the given acks. If all acks have been received then remove it from the list of segmenters.
		 */
		TSharedPtr<FUdpMessageSegmenter> MarkAcksOnSegmenter(int32 MessageId, const TArray<uint32>& Segments, const FUdpMessagingTime InCurrentTime)
		{
			TSharedPtr<FUdpMessageSegmenter> Segmenter = Segmenters.FindRef(MessageId);
			if (Segmenter.IsValid())
			{
				Segmenter->MarkAsAcknowledged(Segments);
				if (Segmenter->IsSendingComplete() && Segmenter->AreAcknowledgementsComplete())
				{
					UE_LOGF(LogUdpMessaging, Verbose, "Segmenter for %ls is now complete. Removing", *NodeId.ToString());
					Segmenters.Remove(MessageId);
				}
			}
			else
			{
				UE_LOGF(LogUdpMessaging, Verbose, "No such segmenter for message %d", MessageId);
			}
			return Segmenter;
		}

		/**
		 * Helper template to allow us to remove types of segments in flight.
		 */
		template <typename PredFunc>
		void RemoveInflightIf(PredFunc&& Pred)
		{
			for (TMap<uint64, FSentData>::TIterator ItRemove = InflightSegments.CreateIterator(); ItRemove; ++ItRemove)
			{
				if (Pred(ItRemove.Value()))
				{
					ItRemove.RemoveCurrent();
				}
			}
		}

		/**
		 * Update the calculated round trip time based on the computed span of inflight packet.
		 */
		void CalculateNewRTT(const FTimespan& NewSpan)
		{
			constexpr double ClockGranularity = 0.005; // "G" = 5ms

			// The computed round trip time for this time.
			const double RTT = NewSpan.GetTotalSeconds();

			if (SmoothedRTT > 0)
			{
				// Calculate the smoothed variance of the RTT so that we can use it in computing a timeout value.
				const double VarianceWeight = 1 / 4.0; // "beta"
				RTTVariance = (1 - VarianceWeight) * RTTVariance + VarianceWeight * FMath::Abs(RTT - SmoothedRTT);

				// Use a smoothed RTT. 1/8 is the value TCP typically uses.
				const double Weight = 1 / 8.0; // "alpha"
				SmoothedRTT = (1 - Weight) * SmoothedRTT + Weight * RTT;
			}
			else
			{
				// Calculate initial values based on the first RTT. 
				SmoothedRTT = RTT;
				RTTVariance = SmoothedRTT / 2;
			}

			Statistics.AverageRTT = FTimespan::FromSeconds(SmoothedRTT);

			// Our transmission timeout becomes the smoothed RTT plus the calculated jitter. 
			RetransmissionTimeout = SmoothedRTT + FMath::Max(ClockGranularity, 4 * RTTVariance);

			static constexpr double MinRTO = 0.020;
			static constexpr double MaxRTO = 3.000;
			RetransmissionTimeout = FMath::Clamp(RetransmissionTimeout, MinRTO, MaxRTO);
		}

		/** 
		 * Compute the new window size for the connection.  We use a AIMD algorithm:
		 * https://en.wikipedia.org/wiki/Additive_increase/multiplicative_decrease
		 */
		void ComputeWindowSize(uint32 NumAcks, uint32 SegmentLoss = 0)
		{
			const uint64 MinimumWindowSize = 64;
			const uint64 MaximumWindowSize = MaxWindowSize;
			if (SegmentLoss == 0)
			{
				// If we did not get any packet loss increase our window size.
				WindowSize = FGenericPlatformMath::Min<uint64>(WindowSize+NumAcks, MaximumWindowSize);
			}
			else
			{
				// In the case of segment loss half our window size to a minimum value.
				WindowSize = FGenericPlatformMath::Max<uint64>(WindowSize/2, MinimumWindowSize);
			}
			Statistics.PacketsAcked += NumAcks;
			Statistics.WindowSize = WindowSize;
		}

		/** Called on transmit. Keeps expiry queues in sync with InflightSegments map. */
		void AddInflight(const FSentData& SentData)
		{
			const int32 MessageId = SentData.MessageId;
			const uint32 SegmentNumber = SentData.SegmentNumber;

			const uint64 Key = Remap(MessageId, SegmentNumber);
			InflightSegments.Add(Key, SentData);

			TRingBuffer<FSegmentSendExpiry>& ExpiryQueue = SentData.bIsReliable ? ReliableExpiry : UnreliableExpiry;
			ExpiryQueue.Emplace(FSegmentSendExpiry{ .TimeSent = SentData.TimeSent,
				.MessageId = MessageId, .SegmentNumber = SegmentNumber });
		}

		/** Undo a previous AddInflight. This frees up send window, but leaves the harmless stale expiry. */
		void RemoveInflight(const FSentData& SentData)
		{
			const int32 MessageId = SentData.MessageId;
			const uint32 SegmentNumber = SentData.SegmentNumber;

			const uint64 Key = Remap(MessageId, SegmentNumber);
			InflightSegments.Remove(Key);
			// We leave stale expiry entries because it's cheap to lazily ignore them later.
		}

		/**
		 * Remove all segments in the inflight buffer that match the given MessageId.
		 */
		uint32 RemoveAllInflightFromMessageId(int32 MessageId)
		{
			int32 BeforeRemove = InflightSegments.Num();
			RemoveInflightIf([MessageId](const FSentData& Data)
			{
				return Data.MessageId == MessageId;
			});
			return BeforeRemove - InflightSegments.Num();
		}

		/**
		 * Make a given MessageId as complete and recompute the desired window size based
		 * on any loss / acks received.
		 */
		void MarkComplete(int32 MessageId, const FUdpMessagingTime InCurrentTime)
		{
			const uint32 AckSegments = RemoveAllInflightFromMessageId(MessageId);
			const uint32 LostSegmentCount = ExpireSegments(InCurrentTime);
			ComputeWindowSize(AckSegments, LostSegmentCount);
		}

		/**
		 * For a given list of segments to acknowledge record how long it took to receive a response and update our
		 * average round trip time. We also use this opportunity to calculate any loss and update our window size.
		 * @return true if the message has been fully acknowledged, false otherwise.
		 */
		bool MarkAcks(int32 MessageId, const TArray<uint32>& Segments, const FUdpMessagingTime InCurrentTime)
		{
			TSharedPtr<FUdpMessageSegmenter> Segmenter = MarkAcksOnSegmenter(MessageId, Segments, InCurrentTime);
			FTimespan MaxSpan(0);
			uint32 Acks = 0;
			for (uint32 SegmentId : Segments)
			{
				const uint64 Id = Remap(MessageId, SegmentId);
				FSentData SentData;
				if (InflightSegments.RemoveAndCopyValue(Id, SentData))
				{
					// Karn's algorithm: retransmitted segments are ineligible for RTT measurement.
					if (Segmenter && !Segmenter->WasRetransmitted(SegmentId))
					{
						MaxSpan = FGenericPlatformMath::Max<FTimespan>(InCurrentTime - SentData.TimeSent, MaxSpan);
					}

					++Acks;
				}
			}

			if (MaxSpan > 0)
			{
				CalculateNewRTT(MaxSpan);
			}

			const uint32 LostSegmentCount = ExpireSegments(InCurrentTime);
			ComputeWindowSize(Acks, LostSegmentCount);

			if (Segmenter)
			{
				return Segmenter->IsSendingComplete() && Segmenter->AreAcknowledgementsComplete();
			}
			else
			{
				return false;
			}
		}

		/**
		 * Mark a segment for retransmission because it was deemed lost.
		 */
		void MarkSegmenterSegmentLoss(const FSentData& Data)
		{
			TSharedPtr<FUdpMessageSegmenter> Segmenter = Segmenters.FindRef(Data.MessageId);
			if (Segmenter.IsValid())
			{
				Segmenter->MarkForRetransmission(Data.SegmentNumber);
			}
		}

		/**
		 * Iterate over all segments and discover any potential segments that may be lost.  A lost segment is determined
		 * using 2 times the average round trip time.  If an ack has not been received in that time frame then it is
		 * lost and we must resend it.
		 */
		uint32 ExpireSegments(const FUdpMessagingTime InCurrentTime)
		{
			SCOPED_MESSAGING_TRACE(FNodeInfo_ExpireSegments);

			auto ExpireUpTo = [this]
			(TRingBuffer<FSegmentSendExpiry>& ExpiryQueue, FUdpMessagingTime ExpiryTime, bool bMarkLoss) -> uint32
			{
				uint32 SegmentsExpired = 0;
				while (ExpiryQueue.Num() && ExpiryQueue.First().TimeSent <= ExpiryTime)
				{
					const FSegmentSendExpiry Expiry = ExpiryQueue.PopFrontValue();

					const uint64 ExpiredKey = Remap(Expiry.MessageId, Expiry.SegmentNumber);
					const FSentData* ExpiredData = InflightSegments.Find(ExpiredKey);

					// This may have already been removed for other reasons; that's OK.
					// If the timestamp doesn't match, we retransmitted; ignore this stale expiry.
					if (!ExpiredData || ExpiredData->TimeSent != Expiry.TimeSent)
					{
						continue;
					}

					if (bMarkLoss)
					{
						MarkSegmenterSegmentLoss(*ExpiredData);
					}

					++SegmentsExpired;
					InflightSegments.Remove(ExpiredKey);
				}

				return SegmentsExpired;
			};

			// Expire reliable segments.
			const FTimespan ReliableTimeout = FTimespan::FromSeconds(RetransmissionTimeout);
			const FUdpMessagingTime ReliableSentExpiryTime = InCurrentTime - ReliableTimeout;
			const uint32 ReliableSegmentsLost = ExpireUpTo(ReliableExpiry, ReliableSentExpiryTime, true);

			// Expire unreliable segments. They are not ack-eliciting, so 1/2 timeout is intended to account for send congestion.
			const FTimespan UnreliableTimeout = ReliableTimeout / 2;
			const FUdpMessagingTime UnreliableSentExpiryTime = InCurrentTime - UnreliableTimeout;
			ExpireUpTo(UnreliableExpiry, UnreliableSentExpiryTime, false);

			Statistics.TotalBytesLost += ReliableSegmentsLost * MaxSegmentSize;
			Statistics.PacketsLost += ReliableSegmentsLost;
			return ReliableSegmentsLost;
		}

		/** Cull old redelivery guards. */
		void ExpireRecentDeliveries()
		{
			const FTimespan RedeliveryGuardband = FTimespan::FromSeconds(10);
			const FUdpMessagingTime ExpiryTime = FUdpMessagingTime::Now() - RedeliveryGuardband;
			while (DeliveryExpiry.Num() && DeliveryExpiry.First().TimeDelivered <= ExpiryTime)
			{
				const FMessageDeliverExpiry Expiry = DeliveryExpiry.PopFrontValue();
				RecentlyDeliveredMessages.Remove(Expiry.MessageId);
			}
		}

		/**
		 * Does this node have any segmenters that can send data for the current time frame.
		 */
		bool HasSegmenterThatCanSend(const FUdpMessagingTime InCurrentTime)
		{
			using FSegmenterTuple = TTuple<int32,TSharedPtr<FUdpMessageSegmenter>>;
			for (FSegmenterTuple& Segmenter : Segmenters)
			{
				if (Segmenter.Value->IsInitialized() && !Segmenter.Value->IsSendingComplete())
				{
					return true;
				}
			}
			return false;
		}

		/**
		 * Do we still have room in our outbound window to send data.
		 */
		bool CanSendSegments() const
		{
			return InflightSegments.Num() < WindowSize;
		}

		int64 SendCapacity() const
		{
			return static_cast<int64>(WindowSize) - static_cast<int64>(InflightSegments.Num());
		}
	};


	/** Structure for inbound segments. */
	struct FInboundSegment
	{
		/** Holds the segment data. */
		TSharedPtr<FArrayReader> Data;

		/** Holds the sender's network endpoint. */
		FIPv4Endpoint Sender;

#if UDPMESSAGINGTRACE_ENABLED
		const FUdpMessagingTime EnqueueTime;
#endif

		/** Creates and initializes a new instance. */
		FInboundSegment(
			const TSharedPtr<FArrayReader>& InData,
			const FIPv4Endpoint& InSender,
			FUdpMessagingTime InEnqueueTime
		)
			: Data(InData)
			, Sender(InSender)
#if UDPMESSAGINGTRACE_ENABLED
			, EnqueueTime(InEnqueueTime)
#endif
		{ }
	};


	/** Structure for outbound messages. */
	struct FOutboundMessage
	{
		/** Holds the serialized message. */
		TSharedPtr<FUdpSerializedMessage> SerializedMessage;

		/** Holds the recipients. */
		TArray<FGuid> RecipientIds;

		EMessageFlags MessageFlags;

#if UDPMESSAGINGTRACE_ENABLED
		const FUdpMessagingTime EnqueueTime;
#endif

		/** Creates and initializes a new instance. */
		FOutboundMessage(
			const TSharedRef<FUdpSerializedMessage>& InSerializedMessage,
			const TArray<FGuid>& InRecipientIds,
			EMessageFlags InFlags,
			FUdpMessagingTime InEnqueueTime
		)
			: SerializedMessage(InSerializedMessage)
			, RecipientIds(InRecipientIds)
			, MessageFlags(InFlags)
#if UDPMESSAGINGTRACE_ENABLED
			, EnqueueTime(InEnqueueTime)
#endif
		{ }
	};

	bool FillDataChunk(FUdpMessageSegment::FDataChunk& OutDataChunk, FNodeInfo& NodeInfo, TSharedPtr<FUdpMessageSegmenter>& Segmenter, int32 MessageId);

	/** Returns the specified segmenter if valid, and attempts to initialize it if necessary. */
	TSharedPtr<FUdpMessageSegmenter>* GetSegmenter(FNodeInfo& NodeInfo, int32 MessageId);

	int32 ClampMaxSegments(int32 MaxSegments, const TSharedPtr<FUdpMessageSegmenter>& Segmenter, const FNodeInfo& NodeInfo) const;

	/** Send one or more segments out for the given message id. Send state is provided by the return value. */
	FSentSegmentInfo SendNextSegmentsForMessageId(FNodeInfo& NodeInfo, FUdpMessageSegment::FHeader& Header, int32 MessageId, int32 MaxSegments = 1);

	/** Remove any nodes we have not heard from in a while. */
	void RemoveDeadNodes();

	/** Update network statistics table that callers can use to gather info about running connections. */
	void UpdateNetworkStatistics();

	/** Send a segmenter stats to listeners  */
	void SendSegmenterStatsToListeners(int32 MessageId, FGuid NodeId, const TSharedPtr<FUdpMessageSegmenter>& Segmenter);

public:

	/**
	 * Creates and initializes a new message processor.
	 *
	 * @param InSocket The network socket used to transport messages.
	 * @param InNodeId The local node identifier (used to detect the unicast endpoint).
	 * @param InMulticastEndpoint The multicast group endpoint to transport messages to.
	 */
	FUdpMessageProcessor(FSocket& InSocket, const FGuid& InNodeId, const FIPv4Endpoint& InMulticastEndpoint);

	/** Virtual destructor. */
	virtual ~FUdpMessageProcessor();

	/**
	 * Add a static endpoint to the processor
	 * @param InEndpoint the endpoint to add
	 */
	void AddStaticEndpoint(const FIPv4Endpoint& InEndpoint);

	/**
	 * Remove a static endpoint from the processor
	 * @param InEndpoint the endpoint to remove
	 */
	void RemoveStaticEndpoint(const FIPv4Endpoint& InEndpoint);

	/**
	 * Get a list of Nodes Ids split by supported Protocol version
	 *
	 * @param Recipients The list of recipients Ids
	 * @return A map of protocol version -> list of node ids for that protocol
	 */
	TMap<uint8, TArray<FGuid>> GetRecipientsPerProtocolVersion(const TArray<FGuid>& Recipients);

	/**
	 * Queues up an inbound message segment.
	 *
	 * @param Data The segment data.
	 * @param Sender The sender's network endpoint.
	 * @return true if the segment was queued up, false otherwise.
	 */
	bool EnqueueInboundSegment(const TSharedPtr<FArrayReader, ESPMode::ThreadSafe>& Data, const FIPv4Endpoint& Sender);

	/**
	 * Queues up an outbound message.
	 *
	 * @param MessageContext The message to serialize and send.
	 * @param Recipients The recipients ids to send to.
	 * @return true if the message was queued up, false otherwise.
	 */
	bool EnqueueOutboundMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext, const TArray<FGuid>& Recipients);

	/**
	 * Get the event used to signal the message processor that work is available.
	 *
	 * @return The event.
	 */
	const TSharedPtr<FEvent, ESPMode::ThreadSafe>& GetWorkEvent() const
	{
		return WorkEvent;
	}

	/**
	 * Waits for all serialization tasks fired by this processor to complete. Expected to be called when the application exit
	 * to prevent serialized (UStruct) object to being use after the UObject system is shutdown.
	 */
	void WaitAsyncTaskCompletion();

	/**
	 * Get the current running network statistics for the given node.
	 */
	FMessageTransportStatistics GetStats(FGuid Node) const;

	void SetShareKnownNodesState(bool bInShareKnownNodes)
	{
		bShareKnownNodes = bInShareKnownNodes;
	}
public:

	// @todo gmp: remove the need for this typedef
	typedef TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe> IMessageAttachmentPtr;

	/**
	 * Returns a delegate that is executed when message data has been received.
	 *
	 * @return The delegate.
	 */
	DECLARE_DELEGATE_ThreeParams(FOnMessageReassembled, FUdpReassembledMessageSharedPtr /*ReassembledMessage*/, const IMessageAttachmentPtr& /*Attachment*/, const FGuid& /*NodeId*/)
	FOnMessageReassembled& OnMessageReassembled()
	{
		return MessageReassembledDelegate;
	}

	/**
	 * Returns a delegate that is executed when a remote node has been discovered.
	 *
	 * @return The delegate.
	 * @see OnNodeLost
	 */
	DECLARE_DELEGATE_OneParam(FOnNodeDiscovered, const FGuid& /*NodeId*/)
	FOnNodeDiscovered& OnNodeDiscovered()
	{
		return NodeDiscoveredDelegate;
	}

	/**
	 * Returns a delegate that is executed when a remote node was closed or timed out.
	 *
	 * @return The delegate.
	 * @see OnNodeDiscovered
	 */
	DECLARE_DELEGATE_OneParam(FOnNodeLost, const FGuid& /*NodeId*/)
	FOnNodeLost& OnNodeLost()
	{
		return NodeLostDelegate;
	}

	/**
	 * Returns a delegate that is executed when a socket error happened.
	 *
	 * @return The delegate.
	 * @note this delegate is broadcasted from the processor thread.
	 */
	DECLARE_DELEGATE(FOnError)
	FOnError& OnError()
	{
		return ErrorDelegate;
	}

	void SetEnableMessageDelegates(bool bShouldEnable)
	{
		bEnableMessageDelegates = bShouldEnable;
	}

	/**
	 * Inbound segment delegate inspector
	 *
	 * @return The delegate.
	 */
	DECLARE_TS_MULTICAST_DELEGATE_FourParams(FOnInboundSegment, const FInboundSegment&, const FGuid&, EUdpMessageSegments, bool)
	FOnInboundSegment& OnInboundSegmentReceived_UdpMessageProcessorThread()
	{
		return InboundSegmentDelegate;
	}
	
	/** 
	 * Returns a delegate that is executed when a socket fails to communicate
	 * upon sending to a target endpoint.
	 * @return The delegate
	 * @note this delegate is broadcasted from the processor thread.
	 */
	DECLARE_DELEGATE_TwoParams(FOnErrorSendingToEndpoint, const FGuid& /*NodeId*/, const FIPv4Endpoint& /*SendersIpAddress*/)
	FOnErrorSendingToEndpoint& OnErrorSendingToEndpoint_UdpMessageProcessorThread()
	{
		return ErrorSendingToEndpointDelegate;
	}

	/**
	 * Returns a delegate that is executed when a socket fails to communicate
	 * upon sending to a target endpoint.
	 * @return The delegate
	 * @note this delegate is broadcasted from the processor thread.
	 */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FCanAcceptEndpoint, const FGuid& /*NodeId*/, const FIPv4Endpoint& /*SendersIpAddress*/)
	FCanAcceptEndpoint& OnCanAcceptEndpoint_UdpMessageProcessorThread()
	{
		return CanAcceptEndpointDelegate;
	}

	TArray<FIPv4Endpoint> GetKnownEndpoints() const;

public:

	//~ FRunnable interface

	virtual FSingleThreadRunnable* GetSingleThreadInterface() override;
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override { }

protected:

	/**
	 * Acknowledges receipt of a message.
	 *
	 * @param MessageId The identifier of the message to acknowledge.
	 * @param NodeInfo Details for the node to send the acknowledgment to.
	 * @todo gmp: batch multiple of these into a single message
	 */
	void AcknowledgeReceipt(int32 MessageId, const FNodeInfo& NodeInfo);

	/**
	 * Calculates the time span that the thread should wait for work.
	 *
	 * @return Wait time.
	 */
	FTimespan CalculateWaitTime() const;

	/** Consumes all inbound segments. */
	void ConsumeInboundSegments();

	/** Consumes all outbound messages. */
	void ConsumeOutboundMessages();

	/**
	 * Filters the specified message segment.
	 *
	 * @param Header The segment header.
	 * @param Data The segment data.
	 * @param Sender The segment sender.
	 * @return true if the segment passed the filter, false otherwise.
	 */
	bool FilterSegment(const FUdpMessageSegment::FHeader& Header, const FIPv4Endpoint& Sender);

	/**
	 * Processes an Abort segment.
	 *
	 * @param Segment The segment to process.
	 * @param NodeInfo Details for the node that sent the segment.
	 */
	void ProcessAbortSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo);

	/**
	 * Processes an Acknowledgement segment.
	 *
	 * @param Segment The segment to process.
	 * @param NodeInfo Details for the node that sent the segment.
	 */
	void ProcessAcknowledgeSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo);

	/**
	 * Processes an AcknowledgmentSegments segment.
	 *
	 * @param Segment The segment to process.
	 * @param NodeInfo Details for the node that sent the segment.
	 */
	void ProcessAcknowledgeSegmentsSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo);

	/**
	 * Processes a Bye segment.
	 *
	 * @param Segment The segment to process.
	 * @param NodeInfo Details for the node that sent the segment.
	 */
	void ProcessByeSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo);

	/**
	 * Processes a Ping segment.
	 *
	 * @param Segment The segment to process.
	 * @param NodeInfo Details for the node that sent the segment.
	 */
	void ProcessDataSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo);

	/**
	 * Processes a Hello segment.
	 *
	 * @param Segment The segment to process.
	 * @param NodeInfo Details for the node that sent the segment.
	 */
	void ProcessHelloSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo);

	/**
	 * Processes a Ping segment.
	 *
	 * @param Segment The segment to process.
	 * @param NodeInfo Details for the node that sent the segment.
	 */
	void ProcessPingSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo);

	/**
	 * Processes a Pong segment.
	 *
	 * @param Segment The segment to process.
	 * @param NodeInfo Details for the node that sent the segment.
	*/
	void ProcessPongSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo);

	/**
	 * Processes a Retransmit segment.
	 *
	 * @param Segment The segment to process.
	 * @param NodeInfo Details for the node that sent the segment.
	 */
	void ProcessRetransmitSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo);

	/**
	 * Processes a Timeout segment.
	 *
	 * @param Segment The segment to process.
	 * @param NodeInfo Details for the node that sent the segment.
	 */
	void ProcessTimeoutSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo);

	/**
	 * Processes a Mesh segment.
	 *
	 * @param Segment The segment to process.
	 * @param NodeInfo Details for the node that sent the segment.
	 */
	void ProcessMeshSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo);

	/**
	 * Processes an unknown segment type.
	 *
	 * @param Segment The segment to process.
	 * @param NodeInfo Details for the node that sent the segment.
	 * @param SegmentType The segment type.
	 */
	void ProcessUnknownSegment(FInboundSegment& Segment, FNodeInfo& NodeInfo, uint8 SegmentType);

	/**
	 * Try to deliver the message to the transport layer.
	 *
	 * @param ReassembledMessage The message to deliver.
	 * @param NodeInfo Details of the node that sent the message.
	 */
	void TryDeliverMessage(const TSharedPtr<FUdpReassembledMessage>& ReassembledMessage, FNodeInfo& NodeInfo);

	/**
	 * Lookup the reassembled message type and try to determine the source struct type.
	 */
	bool LookupAndCacheMessageType(TSharedPtr<FUdpReassembledMessage>& ReassembledMessage);

	/**
	 * Removes the specified node from the list of known remote endpoints.
	 *
	 * @param NodeId The identifier of the node to remove.
	 */
	void RemoveKnownNode(const FGuid& NodeId);


	/** Updates all known remote nodes. */
	void UpdateKnownNodes();

	/**
	 * Updates all segmenters of the specified node.
	 *
	 * @param NodeInfo Details for the node to update.
	 * @return The actual number of bytes written or -1 if error
	 */
	int32 UpdateSegmenters(FNodeInfo& NodeInfo);

	/**
	 * Updates all reassemblers of the specified node.
	 *
	 * @param NodeInfo Details for the node to update.
	 * @return true if the update was successful
	 */
	bool UpdateReassemblers(FNodeInfo& NodeInfo);

	/** Updates nodes per protocol version map */
	void UpdateNodesPerVersion();


protected:

	//~ FSingleThreadRunnable interface

	virtual void Tick() override;

	/* A shallow copy of NodeInfo for use by the logger. */
	struct FShallowNodeInfo
	{
		/** Holds the node's IP endpoint. */
		FIPv4Endpoint Endpoint;

		/** Holds the time at which we last received any segment from this node. */
		FUdpMessagingTime LastSegmentReceivedTime;

		/** Holds the endpoint's node identifier. */
		FGuid NodeId;

		/** Holds the protocol version this node is communicating with */
		uint8 ProtocolVersion;
	};

	/** Return a shallow copy of the known nodes as seen by the processor. This is used and accessible by debugging logging utils. */
	TArray<FShallowNodeInfo> GetKnownNodesAsSnapshot()
	{
		TArray<FShallowNodeInfo> Result;
		for (auto& KnownNodePair : KnownNodes)
		{
			FNodeInfo& NodeInfo = KnownNodePair.Value;
			Result.Add(
				{
					NodeInfo.Endpoint,
					NodeInfo.LastSegmentReceivedTime,
					NodeInfo.NodeId,
					NodeInfo.ProtocolVersion
			});
		}
		return Result;
	}

private:
	friend struct FUdpMessageProcessorLogger;
	friend struct FUdpMessageLoggingImpl;

	/**
	 * Drain a given send queue to the drain limit specified on the parameter listed. Also drain the overflow bucket if requested. Overflow
	 * may occasionally fill from pending acks or work queue overflowing.  
	 */
	TOptional<FSentSegmentInfoArray> DrainQueue(FNodeInfo& NodeInfo, FNodeInfo::FWorkQueue& WorkQueue, int64 DrainCount, const bool bDrainOverflow);

	/**
	 * Refill the given queue with new work based on the processed messages. They will be put back in the queue to be processed later
	 * in the next round to ensure fairness in the sending of data.
	 */
	int32 RequeueWork(FNodeInfo& NodeInfo, FNodeInfo::FWorkQueue& WorkQueue, const FSentSegmentInfoArray& ProcessedMessages);

	/**
	 * Process a given queue.  This calls into DrainQueue and RequeueWork repeatedly until we reach the drain limit which is the minimum
	 * value of of send capacity.  For example, if DrainLimit is 0 then we work until the send capacity is 0. 
	 */
	bool ProcessQueue(FNodeInfo& NodeInfo, FNodeInfo::FWorkQueue& WorkQueue, int64 DrainLimit, int32& BytesSent, bool bShouldDrainOverflowQueue = false);
		
	/** Send our list of known nodes to the known nodes. */
	void SendKnownNodesToKnownNodes();

	/** Do we support sending known nodes to our known nodes. */
	bool CanSendKnownNodesToKnownNodes() const;

	/** Start the message processor thread. */
	void StartThread();

	/** Delegate invoke when plugin phase has been completed. */
	void OnPluginLoadingPhaseComplete(ELoadingPhase::Type LoadingPhase, bool bPhaseSuccessful);

	/** Handles a communication error to a particular endpoint.*/
	void HandleSocketError(const TCHAR* ErrorReference, const FNodeInfo& NodeInfo) const;

	/** Checks all known nodes to see if any have both segmenters and send windows ready to transmit. */
	bool MoreToSend();

	/** Increments and returns the running message ID, bounding it to be non-negative. */
	int32 AssignNextMessageId();

	/** Consume an Outbound Message for processing. */
	void ConsumeOneOutboundMessage(const FOutboundMessage& OutboundMessage);

	/** Holds the queue of outbound messages. */
	TMpscQueue<FInboundSegment> InboundSegments;

	/** Holds the queue of outbound messages. */
	TMpscQueue<FOutboundMessage> OutboundMessages;

	/** Holds the hello sender. */
	TUniquePtr<FUdpMessageBeacon> Beacon;

	/** Holds the delta time between two ticks. */
	FTimespan DeltaTime;

	/** Holds the protocol version that can be communicated in. */
	TArray<uint8> SupportedProtocolVersions;

	/** Mutex protecting access to the Statistics map. */
	mutable FCriticalSection StatisticsCS;

	/** Map that holds latest statistics for network transmission */
	TMap<FGuid, FMessageTransportStatistics> NodeStats;

	/** Mutex protecting access to the NodeVersions map. */
	mutable FCriticalSection NodeVersionCS;

	/** Holds the protocol version of each nodes separately for safe access (NodeId -> Protocol Version). */
	TMap<FGuid, uint8> NodeVersions;

	/** Holds the collection of known remote nodes. */
	TMap<FGuid, FNodeInfo> KnownNodes;

	/** Indicate if we added new nodes during this frame. */
	bool bAddedNewKnownNodes = false;

#if UDPMESSAGINGTRACE_ENABLED
	/** FNodeInfo trace ID. 0 == invalid. */
	static std::atomic<uint16> NextNodeTraceId;

	/** Holds the local node trace ID. */
	uint16 LocalNodeTraceId;
#endif

	/** Holds the local node identifier. */
	FGuid LocalNodeId;

	/** Holds the last sent message number. */
	int32 LastSentMessage;

	/** Holds the multicast endpoint. */
	FIPv4Endpoint MulticastEndpoint;

	/** Holds the network socket used to transport messages. */
	FSocket* Socket;

	/** Holds the socket sender.*/
	TUniquePtr<UE::UdpMessaging::FSocketSender> SocketSender;

	/** Enable sharing of our known nodes. */
	bool bShareKnownNodes = false;

	/** Holds a flag indicating that the thread is stopping. */
	bool bStopping;

	/** Holds a flag indicating if the processor is initialized. */
	bool bIsInitialized;

	/** Holds the thread object. */
	TUniquePtr<FRunnableThread> Thread;

	/** Holds an event signaling that inbound messages need to be processed. */
	TSharedPtr<FEvent, ESPMode::ThreadSafe> WorkEvent;

	/** Holds a delegate to be invoked when a message was received on the transport channel. */
	FOnMessageReassembled MessageReassembledDelegate;

	/** Holds a delegate to be invoked when a network node was discovered. */
	FOnNodeDiscovered NodeDiscoveredDelegate;

	/** Holds a delegate to be invoked when a network node was lost. */
	FOnNodeLost NodeLostDelegate;

	/** Holds a delegate to be invoked when a socket error happen. */
	FOnError ErrorDelegate;

	/** Holds a delegate to be invoked when a socket error occurs sending to a given endpoint. */
	FOnErrorSendingToEndpoint ErrorSendingToEndpointDelegate;

	/** Holds a delegate to be invoked when checking the validity of a given endpoint address. */
	FCanAcceptEndpoint CanAcceptEndpointDelegate;

	/** Holds a delegate to allow logger to inspect inbound segments. */
	FOnInboundSegment InboundSegmentDelegate;

	/** Enable / disable inbound segment delegates. */
	std::atomic<bool> bEnableMessageDelegates = false;

	/** The configured message format (from UUdpMessagingSettings). */
	EUdpMessageFormat MessageFormat;

	/** Stores a cache of discovered UScriptStruct types found during message processing. */
	TMap<FString, TWeakObjectPtr<UScriptStruct>> CachedTypeInfoMap;

	/** Stores messages pending type lookup. */
	TSet<FUdpReassembledMessageSharedPtr> PendingTypeLookup;

	/** How much memory (MB) should be used before periodically logging information about nodes' memory usage. */
	float MemoryLoggingThresholdMB = -1;

	/** Timestamp used to keep track of the last time we logged the active reassemblers. (A negative value will disable logging) */
	double LastReassemblersLogTime = 0.0;

	/** Defines the maximum number of Hello segments that can be dropped before a remote endpoint is considered dead. */
	static const int32 DeadHelloIntervals;

	/** Defines a timespan after which non fully reassembled messages that have stopped receiving segments are dropped. */
	static const FTimespan StaleReassemblyInterval;
};
