// Copyright Epic Games, Inc. All Rights Reserved.

// Implements a very simple CSV style logger that is supplimental to the logging that the engine uses.  When enabled, it will record all inbound, outbound segment data.  This
// is controlled via a CVar.  There is also a variable that will only log trace packets (UdpProcessor.LogOnlyTracedPackets).
// 
#include "UdpMessageProcessorLogger.h"

#include "Containers/Ticker.h"
#include "Containers/SpscQueue.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"

#include "Misc/Guid.h"
#include "Misc/ScopeExit.h"
#include "Misc/Paths.h"
#include "Misc/StringBuilder.h"
#include "Misc/App.h"

#include "Serialization/Archive.h"
#include "Serialization/ArrayReader.h"

#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"

#include "Features/IModularFeatures.h"

#include "INetworkMessagingExtension.h"

#include "Templates/UniquePtr.h"

#include "Shared/UdpMessageSegment.h"

#include "UdpMessageProcessor.h"
#include "UdpMessagingPrivate.h"

struct FNodeInfoStats
{
	/** Holds the node's IP endpoint. */
	FIPv4Endpoint Endpoint;

	/** Holds the time at which we last received any segment from this node. */
	FUdpMessagingTime LastSegmentReceivedTime;

	/** Holds the endpoint's node identifier. */
	FGuid NodeId;

	/** Holds the protocol version this node is communicating with */
	uint8 ProtocolVersion;

	/** This object is always traced. */
	bool bTraced = true;

	/** This stat is always marked complete so it gets processed by the logger. */
	bool bComplete = true;
};

struct FInboundSegmentStat
{
	/** Sender endpoint */
	FIPv4Endpoint InEndpoint;

	/** Guid for the node. */
	FGuid NodeId;

	/** Segment type */
	EUdpMessageSegments SegmentType;

	/** Was this packet filtered.  */
	bool bFiltered = false;
	
	/** Size of the payload. */
	int64 Size;

	/** By default we never trace inbound segment stats. */
	bool bTraced = false;
	
	/** Inbound segments are always complete for the purposes of logging below. */
	bool bComplete = true;
};

namespace UE::UdpMessaging::Private
{
static int32 bLogOnlyTracedObjects = 0;
static FAutoConsoleVariableRef  CVarLogOnlyTraced(TEXT("UdpProcessor.LogOnlyTracedPackets"), bLogOnlyTracedObjects, TEXT("Log only segments and reassemblers that have a tracing enabled."));
	
static INetworkMessagingExtension* GetMessagingStatistics()
{
	IModularFeatures& ModularFeatures = IModularFeatures::Get();
	if (IsInGameThread())
	{
		if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
		{
			return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
		}
	}
	else
	{
		ModularFeatures.LockModularFeatureList();
		ON_SCOPE_EXIT
		{
			ModularFeatures.UnlockModularFeatureList();
		};
		
		if (ModularFeatures.IsModularFeatureAvailable(INetworkMessagingExtension::ModularFeatureName))
		{
			return &ModularFeatures.GetModularFeature<INetworkMessagingExtension>(INetworkMessagingExtension::ModularFeatureName);
		}
	}
		
	ensureMsgf(false, TEXT("Feature %s is unavailable"), *INetworkMessagingExtension::ModularFeatureName.ToString());
	return nullptr;
}

template<typename TStatLogger>
FDelegateHandle Register(TStatLogger*);

template<typename TStatLogger>
void Unregister(TStatLogger*);

template<typename TStatLogger>
FString GetOutputName(TStatLogger*);

template<typename TStatLogger>
FString GetHeader(TStatLogger*);

template<typename TStatType>
FString LogOne(const TStatType&);
	
}; // namespace UE::UdpMessaging::Private

template <typename TStatType>
struct FUdpMessageProcessorLoggerTyped
{
	FUdpMessageProcessorLoggerTyped(const FIPv4Endpoint& Endpoint)
	{
		UE::UdpMessaging::Private::Register(this);
		TickHandler = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FUdpMessageProcessorLoggerTyped<TStatType>::GameTick));
		StartLogging(Endpoint);
	}

	~FUdpMessageProcessorLoggerTyped()
	{
		UE::UdpMessaging::Private::Unregister(this);
		FTSTicker::RemoveTicker(TickHandler);
		GameTick(0);

		Flush();
	}

	void Flush()
	{
		if (CSVArchive.IsValid())
		{
			CSVArchive->Flush();
		}
	}

	void StartLogging(const FIPv4Endpoint& Endpoint)
	{
		FString InstanceName = FApp::GetInstanceName();
		FString EndpointStr = Endpoint.ToString().Replace(TEXT("."), TEXT("_")).Replace(TEXT(":"), TEXT("p"));
		const FString Filename = FString::Printf(TEXT("%s-%s-%s-%s.csv"),*InstanceName, *EndpointStr, *UE::UdpMessaging::Private::GetOutputName(this), *FDateTime::Now().ToString());
		const FString DestDir = FPaths::ProjectLogDir() / TEXT("UdpMessaging");
		const FString FullPath = DestDir / Filename;

		if (!IFileManager::Get().MakeDirectory(*DestDir, true))
		{
			UE_LOGF(LogUdpMessaging, Warning, "Error creating log directory: %ls", *DestDir);
		}

		CSVArchive = TUniquePtr<FArchive>(IFileManager::Get().CreateFileWriter(*FullPath, EFileWrite::FILEWRITE_AllowRead));
		if (CSVArchive)
		{
			UTF8CHAR UTF8BOM[] = { (UTF8CHAR)0xEF, (UTF8CHAR)0xBB, (UTF8CHAR)0xBF };
			CSVArchive->Serialize(&UTF8BOM, UE_ARRAY_COUNT(UTF8BOM) * sizeof(UTF8CHAR));
		}

		FString CSVHeader = TEXT("Time,") + UE::UdpMessaging::Private::GetHeader(this);
		CSVHeader += LINE_TERMINATOR;
		if (CSVArchive.IsValid())
		{
			FTCHARToUTF8 CSVHeaderUTF8(*CSVHeader);
			CSVArchive->Serialize((UTF8CHAR*)CSVHeaderUTF8.Get(), CSVHeaderUTF8.Length() * sizeof(UTF8CHAR));
		}
	}

	void StatUpdated(TStatType Stats)
	{
		FWrappedType WrappedType{FDateTime::Now(), MoveTemp(Stats)};
		AsyncStatQueue.Enqueue(MoveTemp(WrappedType));
	};

	struct FWrappedType
	{
		FDateTime EventTime;
		TStatType StatData;
	};

	bool GameTick(float DeltaTime)
	{
		if (!CSVArchive.IsValid())
		{
			return true;
		}

		FWrappedType Wrapped;
		while (AsyncStatQueue.Dequeue(Wrapped))
		{
			if (!Wrapped.StatData.bComplete && !Wrapped.StatData.bTraced)
			{
				continue;
			}
			if (UE::UdpMessaging::Private::bLogOnlyTracedObjects > 0 && !Wrapped.StatData.bTraced)
			{
				continue;
			}

			FString CSVRow = Wrapped.EventTime.ToString() + TEXT(",") + UE::UdpMessaging::Private::LogOne(Wrapped.StatData);
			CSVRow += LINE_TERMINATOR;
			{
				FTCHARToUTF8 CSVRowUTF8(*CSVRow);
				CSVArchive->Serialize((UTF8CHAR*)CSVRowUTF8.Get(), CSVRowUTF8.Length() * sizeof(UTF8CHAR));
			}
		}
		return true;
	}

	FTSTicker::FDelegateHandle TickHandler;
	TSpscQueue<FWrappedType> AsyncStatQueue;
	TUniquePtr<FArchive> CSVArchive;
};


namespace UE::UdpMessaging::Private
{
	template<typename TStatLogger>
	FDelegateHandle Register(TStatLogger*)
	{
		return {};
	}

	template<typename TStatLogger>
	void Unregister(TStatLogger*)
	{
	}

	// Template specialization for outbound statistics
	template <>
	FDelegateHandle Register(FUdpMessageProcessorLoggerTyped<FOutboundTransferStatistics>* Logger)
	{
		if (INetworkMessagingExtension* Statistics = GetMessagingStatistics())
		{
			return Statistics->OnOutboundTransferUpdatedFromThread().AddRaw(Logger, &FUdpMessageProcessorLoggerTyped<FOutboundTransferStatistics>::StatUpdated);
		}
		return {};
	}
	// Template specialization for inbound statistics
	template <>
	FDelegateHandle Register(FUdpMessageProcessorLoggerTyped<FInboundTransferStatistics>* Logger)
	{
		if (INetworkMessagingExtension* Statistics = GetMessagingStatistics())
		{
			return Statistics->OnInboundTransferUpdatedFromThread().AddRaw(Logger, &FUdpMessageProcessorLoggerTyped<FInboundTransferStatistics>::StatUpdated);
		}
		return {};
	}
	// Template specialization for outbound statistics
	template <>
	void Unregister(FUdpMessageProcessorLoggerTyped<FOutboundTransferStatistics>* Logger)
	{
		if (INetworkMessagingExtension* Statistics = GetMessagingStatistics())
		{
			Statistics->OnOutboundTransferUpdatedFromThread().RemoveAll(Logger);
		}
	}

	// Template specialization for inbound statistics
	template <>
	void Unregister(FUdpMessageProcessorLoggerTyped<FInboundTransferStatistics>* Logger)
	{
		if (INetworkMessagingExtension* Statistics = GetMessagingStatistics())
		{
			Statistics->OnInboundTransferUpdatedFromThread().RemoveAll(Logger);
		}
	}

	template <>
	FString GetOutputName(FUdpMessageProcessorLoggerTyped<FInboundSegmentStat>* Logger)
	{
		return TEXT("InboundSegmentStat");
	}

	template <>
	FString GetOutputName(FUdpMessageProcessorLoggerTyped<FNodeInfoStats>* Logger)
	{
		return TEXT("NodeInfo");
	}

	template <>
	FString GetOutputName(FUdpMessageProcessorLoggerTyped<FOutboundTransferStatistics>* Logger)
	{
		return TEXT("Outbound");
	}
	
	template<>
	FString GetOutputName(FUdpMessageProcessorLoggerTyped<FInboundTransferStatistics>* Logger)
	{
		return TEXT("Inbound");
	}

	template <>
	FString GetHeader(FUdpMessageProcessorLoggerTyped<FInboundSegmentStat>* Logger)
	{
		TStringBuilder<512> Builder;

		Builder.Append(TEXT("InEndpoint,"));
		Builder.Append(TEXT("NodeId,"));
		Builder.Append(TEXT("SegmentType,"));
		Builder.Append(TEXT("Filtered,"));
		Builder.Append(TEXT("Size"));
		return Builder.ToString();
	}

	template <>
	FString GetHeader(FUdpMessageProcessorLoggerTyped<FOutboundTransferStatistics>* Logger)
	{
		TStringBuilder<512> Builder;
		Builder.Append(TEXT("DestinationId,"));
		Builder.Append(TEXT("MessageId,"));
		Builder.Append(TEXT("BytesToSend,"));
		Builder.Append(TEXT("BytesSent,"));
		Builder.Append(TEXT("BytesAcknowledged,"));
		Builder.Append(TEXT("bIsReliable,"));
		Builder.Append(TEXT("bComplete"));
		return Builder.ToString();
	}

	template<>
	FString GetHeader(FUdpMessageProcessorLoggerTyped<FInboundTransferStatistics>* Logger)
	{
		TStringBuilder<512> Builder;
		Builder.Append(TEXT("OriginId,"));
		Builder.Append(TEXT("MessageId,"));
		Builder.Append(TEXT("BytesToReceive,"));
		Builder.Append(TEXT("BytesReceived,"));
		Builder.Append(TEXT("bWasReliablySent,"));
		Builder.Append(TEXT("bComplete"));
		return Builder.ToString();
	}

	template<>
	FString GetHeader(FUdpMessageProcessorLoggerTyped<FNodeInfoStats>* Logger)
	{
		TStringBuilder<512> Builder;
		Builder.Append(TEXT("Endpoint,"));
		Builder.Append(TEXT("LastSegmentReceivedTime,"));
		Builder.Append(TEXT("NodeId,"));
		Builder.Append(TEXT("ProtocolVersion"));
		return Builder.ToString();
	}

	template<>
	FString LogOne(const FInboundSegmentStat& Stat)
	{
		TStringBuilder<512> Builder;
		Builder.Append(Stat.InEndpoint.ToString());
		Builder.Append(TEXT(","));
		Builder.Append(Stat.NodeId.ToString());
		Builder.Append(TEXT(","));

		switch (Stat.SegmentType)
		{
			case EUdpMessageSegments::Abort:
				Builder.Append(TEXT("Abort"));
				break;

			case EUdpMessageSegments::Acknowledge:
				Builder.Append(TEXT("Ack"));
				break;

			case EUdpMessageSegments::AcknowledgeSegments:
				Builder.Append(TEXT("AckSegments"));
				break;

			case EUdpMessageSegments::Bye:
				Builder.Append( TEXT("Bye"));
				break;

			case EUdpMessageSegments::Data:
				Builder.Append( TEXT("Data") );
				break;

			case EUdpMessageSegments::Hello:
				Builder.Append(TEXT("Hello"));
				break;

			case EUdpMessageSegments::Ping:
				Builder.Append(TEXT("Ping"));
				break;

			case EUdpMessageSegments::Pong:
				Builder.Append(TEXT("Pong"));
				break;

			case EUdpMessageSegments::Retransmit:
				Builder.Append(TEXT("Retransmit"));
				break;

			case EUdpMessageSegments::Timeout:
				Builder.Append(TEXT("Timeout"));
				break;

			case EUdpMessageSegments::Mesh:
				Builder.Append(TEXT("Mesh"));
				break;

			default:
				Builder.Append(TEXT("Unknown"));								
		};
		
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.bFiltered));
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.Size));
		return Builder.ToString();
		
	}

	template<>
	FString LogOne(const FNodeInfoStats& Stat)
	{
		TStringBuilder<512> Builder;
		Builder.Append(Stat.Endpoint.ToString());
		Builder.Append(TEXT(","));
		Builder.Appendf(TEXT("%s"), *Stat.LastSegmentReceivedTime.EstimatedLocalDateTime().ToString());
		Builder.Append(TEXT(","));
		Builder.Append(Stat.NodeId.ToString());
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.ProtocolVersion));
		return Builder.ToString();
	}

	template <>
	FString LogOne(const FOutboundTransferStatistics& Stat)
	{
		TStringBuilder<512> Builder;
		Builder.Append(Stat.DestinationId.ToString());
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.MessageId));
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.BytesToSend));
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.BytesSent));
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.BytesAcknowledged));
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.bIsReliable));
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.bComplete));
		return Builder.ToString();
	}
	
	template<>
	FString LogOne(const FInboundTransferStatistics& Stat)
	{
		TStringBuilder<512> Builder;
		Builder.Append(Stat.OriginId.ToString());
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.MessageId));
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.BytesToReceive));
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.BytesReceived));
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.bWasReliablySent));
		Builder.Append(TEXT(","));
		Builder.Append(FString::FromInt(Stat.bComplete));
		return Builder.ToString();
	}

	TMap<FIPv4Endpoint, FUdpMessageProcessorLogger*> Loggers;
	
	void EnableLogging()
	{
		for (TMap<FIPv4Endpoint, FUdpMessageProcessorLogger*>::TIterator It(Loggers); It; ++It)
		{
			UE_LOGF(LogUdpMessaging, Display, "Starting logging for %ls.", *(It->Key.ToString()));
			It->Value->StartLogging();
		}
	}		

	void DisableLogging()
	{
		for (TMap<FIPv4Endpoint, FUdpMessageProcessorLogger*>::TIterator It(Loggers); It; ++It)
		{
			UE_LOGF(LogUdpMessaging, Display, "Stopping logging for %ls.", *(It->Key.ToString()));
			It->Value->StopLogging();
		}		
	}

	void KnownEndpoints()
	{
		for (TMap<FIPv4Endpoint, FUdpMessageProcessorLogger*>::TIterator It(Loggers); It; ++It)
		{
			UE_LOGF(LogUdpMessaging, Display, "Dumping known nodes for %ls.", *(It->Key.ToString()));
			It->Value->DumpKnownNodeInfo();
		}
	}

	FAutoConsoleCommand EnableLoggingCommand(TEXT("UdpProcessor.EnableLogging"), TEXT("Enable Additional Logging for UdpProcessor."), FConsoleCommandDelegate::CreateStatic(&EnableLogging));
	FAutoConsoleCommand DisableLoggingCommand(TEXT("UdpProcessor.DisableLogging"), TEXT("Disable Additional Logging for UdpProcessor."), FConsoleCommandDelegate::CreateStatic(&DisableLogging));

	FAutoConsoleCommand DumpKnownEndpoints(TEXT("UdpProcessor.DumpKnownEndpoints"), TEXT("Disable Additional Logging for UdpProcessor."), FConsoleCommandDelegate::CreateStatic(&KnownEndpoints));
	

} // namespace UE::UdpMessaging::Private


/** Pointer to logging implementation. */
struct FUdpMessageLoggingImpl
{
	FUdpMessageLoggingImpl(const FIPv4Endpoint& Endpoint, FUdpMessageProcessor* InProcessor)
		: Inbound(Endpoint)
		, Outbound(Endpoint)
		, NodeInfo(Endpoint)
		, InboundSegmentStat(Endpoint)
		, Processor(InProcessor)
	{
	}

	~FUdpMessageLoggingImpl()
	{
		Processor->SetEnableMessageDelegates(false);
		
		Processor->OnInboundSegmentReceived_UdpMessageProcessorThread().RemoveAll(this);
	}

	void AttachToProcessor()
	{
		Processor->OnInboundSegmentReceived_UdpMessageProcessorThread().AddRaw(this, &FUdpMessageLoggingImpl::InboundMessage);

		Processor->SetEnableMessageDelegates(true);
	}
	
	void InboundMessage(const FUdpMessageProcessor::FInboundSegment& InMessage, const FGuid& NodeId, EUdpMessageSegments SegmentType, bool bFiltered)
	{
		FInboundSegmentStat Stat = {
			InMessage.Sender,
			NodeId,
			SegmentType,
			bFiltered,
			InMessage.Data ? InMessage.Data->TotalSize() : 0
		};
		InboundSegmentStat.StatUpdated(MoveTemp(Stat));
	}
	
	FUdpMessageProcessorLoggerTyped<FInboundTransferStatistics> Inbound;
	FUdpMessageProcessorLoggerTyped<FOutboundTransferStatistics> Outbound;
	FUdpMessageProcessorLoggerTyped<FNodeInfoStats> NodeInfo;
	FUdpMessageProcessorLoggerTyped<FInboundSegmentStat> InboundSegmentStat;

	FUdpMessageProcessor* Processor = nullptr;
};

FUdpMessageProcessorLogger::FUdpMessageProcessorLogger(FIPv4Endpoint InEndpoint, uint16 PortNo, FUdpMessageProcessor* InProcessor)
	: Processor(InProcessor)
{
	Endpoint = MoveTemp(InEndpoint);
	Endpoint.Port = PortNo;

	UE::UdpMessaging::Private::Loggers.Add(Endpoint, this);
}

FUdpMessageProcessorLogger::~FUdpMessageProcessorLogger()
{
	UE::UdpMessaging::Private::Loggers.Remove(Endpoint);
}

void FUdpMessageProcessorLogger::StartLogging()
{
	if (!Impl)
	{
		Impl = MakeUnique<FUdpMessageLoggingImpl>(Endpoint, Processor);

		DumpKnownNodeInfo();
		
		Impl->NodeInfo.Flush();
		
		// Attach to the inbound and out bound processors. 
		Impl->AttachToProcessor();		
	}
}

void FUdpMessageProcessorLogger::StopLogging()
{
	Impl.Reset();
}

bool FUdpMessageProcessorLogger::IsLogging() const
{
	return Impl.IsValid();
}

void FUdpMessageProcessorLogger::DumpKnownNodeInfo()
{
	auto SnapshotStats = [this](FUdpMessageProcessorLoggerTyped<FNodeInfoStats> & NodeInfo)
	{
		TArray<FUdpMessageProcessor::FShallowNodeInfo> Snapshot = Processor->GetKnownNodesAsSnapshot();
		for (const FUdpMessageProcessor::FShallowNodeInfo& Value : Snapshot)
		{
			FNodeInfoStats Stat;
			Stat.NodeId = Value.NodeId;
			Stat.Endpoint = Value.Endpoint;
			Stat.LastSegmentReceivedTime = Value.LastSegmentReceivedTime;
			Stat.ProtocolVersion = Value.ProtocolVersion;

			NodeInfo.StatUpdated(MoveTemp(Stat));
		}
	};
	
	if (Impl.IsValid())
	{
		SnapshotStats(Impl->NodeInfo);
	}
	else
	{
		FUdpMessageProcessorLoggerTyped<FNodeInfoStats> NodeInfo(Endpoint);
		SnapshotStats(NodeInfo);
	}
}
