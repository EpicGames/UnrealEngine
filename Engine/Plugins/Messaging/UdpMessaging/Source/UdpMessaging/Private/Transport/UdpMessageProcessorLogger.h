// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Templates/UniquePtr.h"
#include "UdpMessageProcessor.h"

struct FUdpMessageLoggingImpl;

struct FUdpMessageProcessorLogger
{
	FUdpMessageProcessorLogger(FIPv4Endpoint InEndpoint, uint16 PortNo, FUdpMessageProcessor* InProcessor);
	~FUdpMessageProcessorLogger();

	/** Start enhanced logging for UdpMessageProcessor */
	void StartLogging();

	/** Stop enhanced logging for UdpMessageProcessor */
	void StopLogging();

	/** Report the current logging state. */
	bool IsLogging() const;

	/** Dump a report of the current known nodes from the system. */
	void DumpKnownNodeInfo();
	
private:
	FIPv4Endpoint Endpoint;
		
	FUdpMessageProcessor* Processor;
	TUniquePtr<FUdpMessageLoggingImpl> Impl;
};


