// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Transport/UdpReassembledMessage.h"


/**
 * Implements a re-sequencer for messages received over the UDP transport.
 */
class FUdpMessageResequencer
{
public:

	/** Default constructor. */
	FUdpMessageResequencer() { }

	/**
	 * Creates and initializes a new message resequencer.
	 *
	 * @param InWindowSize The maximum resequencing window size.
	 */
	FUdpMessageResequencer(uint16 InWindowSize)
		: NextSequence(1)
	{ }

public:

	/**
	 * Gets the next expected sequence number.
	 *
	 * @return Next sequence.
	 */
	uint64 GetNextSequence() const
	{
		return NextSequence;
	}

	/**
	 * Extracts the next available message in the sequence.
	 *
	 * @return true if a message was returned, false otherwise.
	 */
	bool Pop(TSharedPtr<FUdpReassembledMessage, ESPMode::ThreadSafe>& OutMessage)
	{
		if (MessageHeap.HeapTop()->GetSequence() != NextSequence)
		{
			return false;
		}

		MessageHeap.HeapPop(OutMessage, FSequenceComparer());
		NextSequence++;

		return true;
	}

	/**
	 * Resequences the specified message.
	 *
	 * @param Message The message to resequence.
	 * @return true if the message is in sequence, false otherwise.
	 */
	bool Resequence(const TSharedPtr<FUdpReassembledMessage, ESPMode::ThreadSafe>& Message)
	{
		MessageHeap.HeapPush(Message, FSequenceComparer());

		return (Message->GetSequence() == NextSequence);
	}

	/** Resets the re-sequencer. */
	void Reset()
	{
		MessageHeap.Reset();
		NextSequence = 1;
	}

private:

	/** Helper for ordering messages by their sequence numbers. */
	struct FSequenceComparer
	{
		bool operator()(const TSharedPtr<FUdpReassembledMessage, ESPMode::ThreadSafe>& A, const TSharedPtr<FUdpReassembledMessage, ESPMode::ThreadSafe>& B) const
		{
			return A->GetSequence() < B->GetSequence();
		}
	};

private:

	/** Holds the next expected sequence number. */
	uint64 NextSequence;

	/** Holds the messages that need to be resequenced. */
	TArray<TSharedPtr<FUdpReassembledMessage, ESPMode::ThreadSafe>> MessageHeap;
};
