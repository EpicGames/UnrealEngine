// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transport/UdpMessageSegmenter.h"
#include "UdpMessagingPrivate.h"

#include "Transport/UdpSerializedMessage.h"


/* FUdpMessageSegmenter structors
 *****************************************************************************/

FUdpMessageSegmenter::FUdpMessageSegmenter(
	const TSharedRef<FUdpSerializedMessage>& InSerializedMessage,
	uint16 InSegmentSize
)
	: SegmentSize(InSegmentSize)
	, SerializedMessage(InSerializedMessage)
{
	check(InSegmentSize > GetDataSegmentHeaderSize());
}


FUdpMessageSegmenter::~FUdpMessageSegmenter()
{
	if (MessageReader != nullptr)
	{
		delete MessageReader;
	}
}


/* FUdpMessageSegmenter interface
 *****************************************************************************/

int64 FUdpMessageSegmenter::GetMessageSize() const
{
	if (MessageReader == nullptr)
	{
		return 0;
	}

	return MessageReader->TotalSize();
}


bool FUdpMessageSegmenter::GetNextPendingSegment(TArray<uint8>& OutData, uint32& OutSegment) const
{
	if (MessageReader == nullptr)
	{
		return false;
	}

	if (TConstSetBitIterator<> It(PendingSendSegments, EarliestUnackedSegmentId); It)
	{
		OutSegment = It.GetIndex();

		uint64 SegmentOffset = static_cast<uint64>(OutSegment) * GetSegmentPayloadSize();
		uint64 ActualSegmentSize = MessageReader->TotalSize() - SegmentOffset;

		if (ActualSegmentSize > GetSegmentPayloadSize())
		{
			ActualSegmentSize = GetSegmentPayloadSize();
		}

		OutData.Reset(ActualSegmentSize);
		OutData.AddUninitialized(ActualSegmentSize);

		MessageReader->Seek(SegmentOffset);
		MessageReader->Serialize(OutData.GetData(), ActualSegmentSize);

		//FMemory::Memcpy(OutData.GetTypedData(), Message->GetTypedData() + SegmentOffset, ActualSegmentSize);

		return true;
	}

	return false;
}


bool FUdpMessageSegmenter::GetPendingSegment(uint32 InSegment, TArray<uint8>& OutData) const
{
	if (MessageReader == nullptr)
	{
		return false;
	}

	// Max segment number for protocol 12 is INT32_MAX, if increased, this will need changing
	if (InSegment < (uint32)PendingSendSegments.Num() && PendingSendSegments[InSegment])
	{
		uint64 SegmentOffset = static_cast<uint64>(InSegment) * GetSegmentPayloadSize();
		uint64 ActualSegmentSize = MessageReader->TotalSize() - SegmentOffset;

		if (ActualSegmentSize > GetSegmentPayloadSize())
		{
			ActualSegmentSize = GetSegmentPayloadSize();
		}

		OutData.Reset(ActualSegmentSize);
		OutData.AddUninitialized(ActualSegmentSize);

		MessageReader->Seek(SegmentOffset);
		MessageReader->Serialize(OutData.GetData(), ActualSegmentSize);

		return true;
	}

	return false;
}


bool FUdpMessageSegmenter::TryInitialize()
{
	if (!ensureMsgf(!IsInitialized(), TEXT("Tried to initialize an already initialized segmenter")))
	{
		return false;
	}

	if (SerializedMessage->GetState() != EUdpSerializedMessageState::Complete)
	{
		return false;
	}

	MessageReader = SerializedMessage->CreateReader();
	PendingSendSegmentsCount = (MessageReader->TotalSize() + GetSegmentPayloadSize() - 1) / GetSegmentPayloadSize();
	PendingSendSegments.Init(true, PendingSendSegmentsCount);
	if (EnumHasAnyFlags(GetMessageFlags(), EMessageFlags::Reliable))
	{
		AcknowledgeSegments.Init(false, PendingSendSegmentsCount);
	}
	else
	{
		// Acks for unreliable messages are always zero
		AcknowledgeSegments.Init(false, 0);
	}
	AcknowledgeSegmentsCount = 0;

	return true;
}


bool FUdpMessageSegmenter::IsMessageSerializationDone() const
{
	return SerializedMessage == nullptr || SerializedMessage->GetState() != EUdpSerializedMessageState::Incomplete;
}


bool FUdpMessageSegmenter::IsInvalid() const
{
	return (SerializedMessage->GetState() == EUdpSerializedMessageState::Invalid);
}


uint8 FUdpMessageSegmenter::GetProtocolVersion() const
{
	return SerializedMessage->GetProtocolVersion();
}


EMessageFlags FUdpMessageSegmenter::GetMessageFlags() const
{
	return SerializedMessage->GetFlags();
}

void FUdpMessageSegmenter::MarkAsSent(uint32 SegmentId)
{
	if (SegmentId < (uint32)PendingSendSegments.Num() && PendingSendSegments[SegmentId])
	{
		--PendingSendSegmentsCount;
		PendingSendSegments[SegmentId] = false;
		UE_LOGF(LogUdpMessaging, Verbose, "Marking segment %d of %d as sent (%d outstanding)", SegmentId+1, PendingSendSegments.Num(), PendingSendSegmentsCount);
	}
}

void FUdpMessageSegmenter::MarkAsSent(const TArray<uint32>& Segments)
{
	for (uint32 Segment : Segments)
	{
		MarkAsSent(Segment);
	}
}

void FUdpMessageSegmenter::MarkAsAcknowledged(uint32 Segment)
{
	// Mark this segment as acknowledged. There's a chance segments could be acknowledged
	// twice so we need to check state
	if (Segment < (uint32)AcknowledgeSegments.Num() && !AcknowledgeSegments[Segment])
	{
		++AcknowledgeSegmentsCount;
		AcknowledgeSegments[Segment] = true;
		UE_LOGF(LogUdpMessaging, Verbose, "Marked segment %d of %d as acknowledged (%d outstanding)", Segment + 1, AcknowledgeSegments.Num(), AcknowledgeSegments.Num() - AcknowledgeSegmentsCount);

		if (Segment == EarliestUnackedSegmentId)
		{
			const int32 NextUnackedId = AcknowledgeSegments.FindFrom(false, EarliestUnackedSegmentId);
			EarliestUnackedSegmentId = (NextUnackedId == INDEX_NONE) ? AcknowledgeSegments.Num() : NextUnackedId;
		}
	}

	// We may have queued a segment to be resent, if so there's now no need to resend it
	if (Segment < (uint32)PendingSendSegments.Num() && PendingSendSegments[Segment])
	{
		--PendingSendSegmentsCount;
		PendingSendSegments[Segment] = false;
		UE_LOGF(LogUdpMessaging, Verbose, "Received acknowledgment for segment %d that was queued or requeued for transmission. Will skip send", Segment + 1);
	}
}

void FUdpMessageSegmenter::MarkAsAcknowledged(const TArray<uint32>& Segments)
{
	if (ensure(EnumHasAnyFlags(GetMessageFlags(), EMessageFlags::Reliable)))
	{
		for (const auto& Segment : Segments)
		{
			MarkAsAcknowledged(Segment);
		}
	}
}

void FUdpMessageSegmenter::MarkForRetransmission(uint32 SegmentId)
{
	if (SegmentId < (uint32)PendingSendSegments.Num() && !PendingSendSegments[SegmentId])
	{
		UE_LOGF(LogUdpMessaging, Verbose, "Marking segment %d of %d for retransmission", SegmentId+1, PendingSendSegments.Num());

		++PendingSendSegmentsCount;
		PendingSendSegments[SegmentId] = true;

		RetransmittedSegments.Add(SegmentId);
		++RetransmitCount;

		// Note - we don't need to clear acknowledgments. If any segment is in transit and acknowledged after this
		// call we'll do that and if possible stop the pending send, and if not we don't need to wait for the ack
	}
}

void FUdpMessageSegmenter::MarkForRetransmission(const TArray<uint16>& Segments)
{
	for (uint16 Segment : Segments)
	{
		MarkForRetransmission(Segment);
	}
}

/**
 * Marks the entire message for retransmission.
 */
void FUdpMessageSegmenter::MarkForRetransmission()
{
	UE_LOGF(LogUdpMessaging, Verbose, "Marking all %d segments for retransmission", PendingSendSegments.Num());

	for (TBitArray<>::FConstIterator BIt(AcknowledgeSegments, EarliestUnackedSegmentId); BIt; ++BIt)
	{
		const bool bAcknowledged = BIt.GetValue();
		const int32 SegmentId = BIt.GetIndex();
		if (!bAcknowledged && !PendingSendSegments[SegmentId])
		{
			RetransmittedSegments.Add(SegmentId);
			++RetransmitCount;
		}
	}

	// mark all segments to be resent and clear any pending state
	PendingSendSegments.Init(true, PendingSendSegments.Num());
	PendingSendSegmentsCount = PendingSendSegments.Num();
	
	// Note - we don't need to clear acknowledgments. If any segment is in transit and acknowledged after this
	// call we'll do that and if possible stop the pending send, and if not we don't need to wait for the ack
}

bool FUdpMessageSegmenter::WasRetransmitted(uint32 SegmentId) const
{
	if (RetransmitCount == 0)
	{
		return false;
	}

	return RetransmittedSegments.Contains(SegmentId);
}

#if UDPMESSAGINGTRACE_ENABLED
void FUdpMessageSegmenter::EmitPostInitializeTraceEvents(uint16 InSenderTraceId, uint16 InRecipientTraceId, int32 InMessageId)
{
	const TVariant<FName, FTopLevelAssetPath> TypeInfo(
		TInPlaceType<FTopLevelAssetPath>(), SerializedMessage->GetTraceMetadata().TypeInfo);

	TRACE_UDPMESSAGING_MESSAGE_TYPEINFO(
		InSenderTraceId,
		InRecipientTraceId,
		InMessageId,
		TypeInfo
	);

	TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
		SerializedMessage->GetTraceMetadata().SerializationStartTime.Cycles,
		InSenderTraceId,
		InRecipientTraceId,
		InMessageId,
		UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::BeginSerialize
	);

	TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
		SerializedMessage->GetTraceMetadata().SerializationEndTime.Cycles,
		InSenderTraceId,
		InRecipientTraceId,
		InMessageId,
		UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::EndSerialize
	);

	TRACE_UDPMESSAGING_MESSAGE_SUMMARY(
		InSenderTraceId,
		InRecipientTraceId,
		InMessageId,
		GetMessageSize(),
		GetSegmentCount(),
		GetMessageFlags()
	);
}
#endif // #if UDPMESSAGINGTRACE_ENABLED
