// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Shared/UdpMessageSegment.h"
#include "Shared/UdpMessagingTime.h"
#include "Templates/SharedPointer.h"
#include "Trace/UdpMessagingTrace.h"


// IMessageContext forward declaration
enum class EMessageFlags : uint32;

class FArchive;
class FUdpSerializedMessage;


/**
 * Implements a message segmenter.
 *
 * This class breaks up a message into smaller sized segments that fit into UDP datagrams.
 * It also tracks the segments that still need to be sent.
 */
class FUdpMessageSegmenter
{
public:
	/**
	 * Creates and initializes a new message segmenter.
	 *
	 * Segment size includes UdpMessaging headers (\see GetDataSegmentHeaderSize),
	 * but not UDP/IP headers.
	 *
	 * @param InMessage The serialized message to segment.
	 * @param InSegmentSize The target total segment size to divide messages into.
	 */
	FUdpMessageSegmenter(
		const TSharedRef<FUdpSerializedMessage>& InSerializedMessage,
		uint16 InSegmentSize
	);

	/** Destructor. */
	~FUdpMessageSegmenter();

public:

	/**
	 * Gets the total size of the message in bytes.
	 *
	 * @return Message size.
	 */
	int64 GetMessageSize() const;

	/**
	 * Gets the next pending segment.
	 *
	 * @param OutData Will hold the segment data.
	 * @param OutSegment Will hold the segment number.
	 * @return true if a segment was returned, false if there are no more pending segments.
	 */
	bool GetNextPendingSegment(TArray<uint8>& OutData, uint32& OutSegment) const;

	/**
	 * Gets the pending segment at.
	 *
	 * @param InSegment the segment number we are requesting the data for.
	 * @param OutData Will hold the segment data.
	 * @return true if a segment was returned, false if that segment is no longer pending or the segment number is invalid.
	 */
	bool GetPendingSegment(uint32 InSegment, TArray<uint8>& OutData) const;


	/**
	 * Get the pending segments array.
	 * @return the list of pending segments flags.
	 */
	const TBitArray<>& GetPendingSendSegments() const
	{
		return PendingSendSegments;
	}

	/**
	 * Gets the number of segments that haven't been received yet.
	 *
	 * @return Number of pending segments.
	 */
	uint32 GetPendingSendSegmentsCount() const
	{
		return PendingSendSegmentsCount;
	}

	/**
	 * Get the array of acknowledged segments. True represents acknowledgement. This array will be empty for unreliable messages
	 * @return the list of segments.
	 */
	const TBitArray<>& GetAcknowledgedSegments() const
	{
		return AcknowledgeSegments;
	}

	/**
	 * Gets the number of segments that have been acknowledged. Always zero for unreliable messages
	 *
	 * @return Number of acknowledged segments.
	 */
	uint32 GetAcknowledgedSegmentsCount() const
	{
		return AcknowledgeSegmentsCount;
	}

	/**
	 * Gets the total number of segments that make up the message.
	 *
	 * @return Segment count.
	 */
	uint32 GetSegmentCount() const
	{
		return PendingSendSegments.Num();
	}

	/**
	 * Gets the total segment size including headers.
	 *
	 * @return Segment size in bytes.
	 */
	uint16 GetTotalSegmentSize() const
	{
		return SegmentSize;
	}

	/**
	 * Gets the size of the headers that are prepended to each segment.
	 *
	 * @return Header size in bytes.
	 */
	static constexpr uint16 GetDataSegmentHeaderSize()
	{
		return FUdpMessageSegment::FHeader::SerializedBytes
			+ FUdpMessageSegment::FDataChunk::FixedSerializedBytes;
	}

	/**
	 * Gets the number of actual message data bytes per segment, excluding headers (goodput).
	 *
	 * @return Segment payload size in bytes.
	 */
	uint16 GetSegmentPayloadSize() const
	{
		return SegmentSize - GetDataSegmentHeaderSize();
	}

	/**
	 * Attempt to initialize an uninitialized segmenter.
	 *
	 * @return true if the segmenter was uninitialized and serialization is complete, false otherwise.
	 */
	bool TryInitialize();

	/**
	 * Checks whether all segments have been sent.
	 *
	 * @return true if all segments were sent, false otherwise.
	 */
	bool IsSendingComplete() const
	{
		return (PendingSendSegmentsCount == 0);
	}

	/**
	 * Checks whether all outstanding acknowledgments have been received. Always true for an unreliable message once its sent
	 *
	 * @return true if all acknowledgments are received
	 */
	bool AreAcknowledgementsComplete() const
	{
		return (AcknowledgeSegmentsCount == AcknowledgeSegments.Num());
	}

	/**
	 * Checks whether this segmenter has been initialized.
	 *
	 * @return true if it is initialized, false otherwise.
	 */
	bool IsInitialized() const
	{
		return (MessageReader != nullptr);
	}

	/**
	 * Checks whether this segmenter is invalid.
	 *
	 * @return true if the segmenter is invalid, false otherwise.
	 */
	bool IsInvalid() const;

	/** Return the Protocol Version for this segmenter.	*/
	uint8 GetProtocolVersion() const;

	/** @return the message flags. */
	EMessageFlags GetMessageFlags() const;

	/**
	* Marks the given segment id as sent.
	*
	* @param Segment id.
	*/
	void MarkAsSent(uint32 SegmentId);

	/**
	* Marks the specified segments as sent
	*
	* @param Segments The acknowledged segments.
	*/
	void MarkAsSent(const TArray<uint32>& Segments);

	/**
	* Marks the given segment id as acknowledged.
	*
	* @param Segment id.
	*/
	void MarkAsAcknowledged(uint32 SegmentId);

	/**
	* Marks the specified segments as acknowledged.
	*
	* @param Segments The acknowledged segments.
	*/
	void MarkAsAcknowledged(const TArray<uint32>& Segments);

	/**
	 * Marks the entire message for retransmission. This does not reset acknowledgements as we only need an acknowledgment for a segement once
	 */
	void MarkForRetransmission();

	/**
	 * Marks a given segment id for retransmission
	 */
	void MarkForRetransmission(uint32 SegmentId);

	/**
	 * Marks the specified segments for retransmission.
	 *
	 * @param Segments The data segments to retransmit.
	 * @note this function is kept for legacy reasons to be used with FRetransmitChunk which still encodes its segment count on uint16. FRetransmitChunk aren't used in protocol 12 and newer.
	 */
	void MarkForRetransmission(const TArray<uint16>& Segments);

	/** Return whether a segment has been retransmitted. */
	bool WasRetransmitted(uint32 SegmentId) const;

	/**
	 * Checks whether the serialization of the message to segment and send completed. The serialization may be performing asynchronously and can may succeed or fail. The message cannot be segmented
	 * and sent until it was serialized sucessfully.
	 * @return true if the message is serialization is done, false if it still serializing.
	 * @see TryInitialize()
	 * @see IsInvalid()
	 * @see IsInitialized()
	 */
	bool IsMessageSerializationDone() const;

#if UDPMESSAGINGTRACE_ENABLED
	/** Once serialization completes, the processor passes in context we use to record facts about this message. */
	void EmitPostInitializeTraceEvents(uint16 InSenderTraceId, uint16 InRecipientTraceId, int32 InMessageId);
#endif

private:
	/** Used to seek and output portions of SerializedMessage. */
	FArchive* MessageReader = nullptr;

	/** Holds an array of bits where true indicates which segments still need to be sent. */
	TBitArray<> PendingSendSegments;

	/** Holds an array of bits where true indicates a segment has been acknowledged. Will always zero-length for an unreliable segment. */
	TBitArray<> AcknowledgeSegments;

	/** Holds the number of segments that haven't been sent yet. */
	uint32 PendingSendSegmentsCount = 0;

	/** Holds the number of segments that haven't been acknowledged yet. Will always be zero for an unreliable message */
	uint32 AcknowledgeSegmentsCount = 0;

	/** Holds the earliest segment for which we have not received an acknowledgment. Useful to narrow search. */
	int32 EarliestUnackedSegmentId = 0;

	/** Holds the segment size (including the FHeader and FDataChunk fields). */
	uint16 SegmentSize = 0;

	/** Holds the number of retransmits. */
	uint16 RetransmitCount = 0;

	/** Holds the message. */
	TSharedPtr<FUdpSerializedMessage, ESPMode::ThreadSafe> SerializedMessage;

	/** Keep a record of any segments we've retransmitted. */
	TSet<uint32> RetransmittedSegments;
};
