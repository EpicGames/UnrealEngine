// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transport/UdpMessageTransport.h"

#include "HAL/IConsoleManager.h"

#include "Common/UdpSocketBuilder.h"
#include "Common/UdpSocketReceiver.h"
#include "IMessageTransportHandler.h"
#include "Async/Async.h"

#include "Shared/UdpMessagingSettings.h"
#include "Tasks/Task.h"
#include "Tasks/TaskConcurrencyLimiter.h"
#include "Trace/UdpMessagingTrace.h"
#include "Transport/UdpReassembledMessage.h"
#include "Transport/UdpDeserializedMessage.h"
#include "Transport/UdpMessageProcessor.h"

#if PLATFORM_DESKTOP
#include "UdpMessageProcessorLogger.h"
#endif


TAutoConsoleVariable<int32> CVarMaxRetriesForBadEndpoint(
	TEXT("MessageBus.UDP.MaxRetriesForBadEndpoint"),
	5,
	TEXT("The maximum number of retries that will be attempted when a socket connection fails to reach an endpoint."),
	ECVF_Default
);

TAutoConsoleVariable<int32> CVarBadEndpointPeriod(
	TEXT("MessageBus.UDP.BadEndpointPeriod"),
	60,
	TEXT("The period of time, in seconds, between endpoint socket errors to be considered a bad endpoint."),
	ECVF_Default
);

TAutoConsoleVariable<bool> CVarEndpointDenyListEnabled(
	TEXT("MessageBus.UDP.EndpointDenyListEnabled"),
	true,
	TEXT("Specifies if the endpoint deny list is enabled. If enabled, a problematic endpoint will be tagged for possible exclusion from communication.")
	TEXT("The maximum attempts allowed is determined by MessageBus.UDP.MaxRetriesForBadEndpoint"),
	ECVF_Default
);

namespace UE::UdpMessageTransport::Private
{

struct FDenyList
{
	struct FDenyCandidate
	{
		int32 EndpointFailureCount = 0;
		FDateTime LastFailTime = FDateTime::UtcNow();
	};

	FDenyList() :
		ClearDenyList(TEXT("MessageBus.UDP.ClearDenyList"), TEXT("Clear the UDP socket deny list."),
					  FConsoleCommandDelegate::CreateRaw(this, &FDenyList::DoClearDenyCandidateList))
	{
	}

	void DoClearDenyCandidateList()
	{
		FScopeLock DenyListLock(&DenyListCS);
		DenyCandidateList.Reset();
	}

	bool ShouldAllowEndpoint(const FGuid& EndpointNodeId) const
	{
		FScopeLock DenyListLock(&DenyListCS);
		const FDenyCandidate* DenyCandidate = DenyCandidateList.Find(EndpointNodeId);
		if (DenyCandidate && DenyCandidate->EndpointFailureCount > CVarMaxRetriesForBadEndpoint.GetValueOnAnyThread())
		{
			return false;
		}
		return true;
	}

	void HandleEndpointError(const FGuid& EndpointId)
	{
		FScopeLock DenyListLock(&DenyListCS);
		FDenyCandidate& DenyCandidate = DenyCandidateList.FindOrAdd(EndpointId);
		FDateTime CurrentTime = FDateTime::UtcNow();
		if ((CurrentTime - DenyCandidate.LastFailTime).GetSeconds() < CVarBadEndpointPeriod.GetValueOnAnyThread())
		{
			DenyCandidate.EndpointFailureCount++;
		}
		else
		{
			DenyCandidate.EndpointFailureCount = 0;
		}
		DenyCandidate.LastFailTime = CurrentTime;
	}

	/** Mutex protecting access to the DenyList map.. */
	mutable FCriticalSection DenyListCS;

	/** Deny list of node ids not allowed to talk to UDP processor */
	TMap<FGuid, FDenyCandidate> DenyCandidateList;

	/** Console command to reset the deny list */
	FAutoConsoleCommand ClearDenyList;
};

static FDenyList GlobalDenyCandidateTable;
}

FUdpMessageTransport::FUdpMessageTransport(const FIPv4Endpoint& InUnicastEndpoint, const FIPv4Endpoint& InMulticastEndpoint, TArray<FIPv4Endpoint> InStaticEndpoints, TArray<FString> InExcludedEndpointsAsString, uint8 InMulticastTtl)
	: MessageProcessor(nullptr)
	, MulticastEndpoint(InMulticastEndpoint)
	, MulticastReceiver(nullptr)
	, MulticastSocket(nullptr)
	, MulticastTtl(InMulticastTtl)
	, SocketSubsystem(ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM))
	, TransportHandler(nullptr)
	, UnicastEndpoint(InUnicastEndpoint)
#if PLATFORM_DESKTOP
	, UnicastReceiver(nullptr)
	, UnicastSocket(nullptr)
#endif
	, StaticEndpoints(MoveTemp(InStaticEndpoints))
	, ExcludedEndpoints(MoveTemp(InExcludedEndpointsAsString))
{
}


FUdpMessageTransport::~FUdpMessageTransport()
{
	StopTransport();
}

void FUdpMessageTransport::OnAppPreExit()
{
	if (MessageProcessor)
	{
#if PLATFORM_DESKTOP
		if (MessageLogger)
		{
			MessageLogger->StopLogging();
		}
#endif
		MessageProcessor->WaitAsyncTaskCompletion();
	}
}

void FUdpMessageTransport::AddStaticEndpoint(const FIPv4Endpoint& InEndpoint)
{
	bool bAlreadyInSet = false;
	StaticEndpoints.Add(InEndpoint, &bAlreadyInSet);
	if (MessageProcessor && !bAlreadyInSet)
	{
		MessageProcessor->AddStaticEndpoint(InEndpoint);
	}
	UE_LOGF(LogUdpMessaging, Verbose, "Added StaticEndpoint at %ls", *InEndpoint.ToString());

}


void FUdpMessageTransport::RemoveStaticEndpoint(const FIPv4Endpoint& InEndpoint)
{
	int32 NbRemoved = StaticEndpoints.Remove(InEndpoint);
	if (MessageProcessor && NbRemoved > 0)
	{
		MessageProcessor->RemoveStaticEndpoint(InEndpoint);
	}
	UE_LOGF(LogUdpMessaging, Verbose, "Removed StaticEndpoint at %ls", *InEndpoint.ToString());
}

/* IMessageTransport interface
 *****************************************************************************/

FName FUdpMessageTransport::GetDebugName() const
{
	return "UdpMessageTransport";
}

TArray<FIPv4Endpoint> FUdpMessageTransport::GetKnownEndpoints() const
{
	if (MessageProcessor)
	{
		return MessageProcessor->GetKnownEndpoints();
	}
	return {};
}

FMessageTransportStatistics FUdpMessageTransport::GetLatestStatistics(FGuid NodeId) const
{
	if (MessageProcessor)
	{
		return MessageProcessor->GetStats(NodeId);
	}
	return {};
}

TArray<FIPv4Endpoint> FUdpMessageTransport::GetListeningAddresses() const
{
	TArray<FIPv4Endpoint> Listeners;
#if PLATFORM_DESKTOP
	Listeners.Add(UnicastEndpoint);
#endif

#if PLATFORM_SUPPORTS_UDP_MULTICAST_GROUP
	Listeners.Add(MulticastEndpoint);
#endif

	return Listeners;
}

namespace UE::UdpMessaging::Private
{

void DoJoinMulticastGroup(const TSharedRef<FInternetAddr>& MulticastAddr, const TSharedPtr<FInternetAddr>& IpAddr, FSocket* MulticastSocket)
{
	if (!IpAddr.IsValid())
	{
		return;
	}
	const bool bJoinedGroup = MulticastSocket->JoinMulticastGroup(*MulticastAddr, *IpAddr);
	if (bJoinedGroup)
	{
		UE_LOGF(LogUdpMessaging, Display, "Added local interface '%ls' to multicast group '%ls'",
			   *IpAddr->ToString(false), *MulticastAddr->ToString(true));
	}
	else
	{
		UE_LOGF(LogUdpMessaging, Warning, "Failed to join multicast group '%ls' on detected local interface '%ls'",
			   *MulticastAddr->ToString(true), *IpAddr->ToString(false));
	}
}

void JoinedToGroup(const FIPv4Endpoint& UnicastEndpoint, const FIPv4Endpoint& MulticastEndpoint, FSocket* MulticastSocket)
{
#if PLATFORM_SUPPORTS_UDP_MULTICAST_GROUP
	TSharedRef<FInternetAddr> MulticastAddr = MulticastEndpoint.ToInternetAddr();
	if (UnicastEndpoint.Address == FIPv4Address::Any)
	{
		TArray<TSharedPtr<FInternetAddr>> LocalAdapterAddresses;
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalAdapterAddresses(LocalAdapterAddresses);

		const bool bShouldBindToLocalHostAddr = LocalAdapterAddresses.Num() == 0;
		
		// The loopback address is not included in the call to GetLocalAdapterAddresses. Make sure we add it so that loopback address is included in the group.
		TSharedRef<FInternetAddr> Loopback = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();
		Loopback->SetLoopbackAddress();
		LocalAdapterAddresses.Add(Loopback.ToSharedPtr());

		// Now iterate over all addresss and join the multicast group.
		for (const TSharedPtr<FInternetAddr>& LocalIp : LocalAdapterAddresses)
		{
			DoJoinMulticastGroup(MulticastAddr, LocalIp, MulticastSocket);
		}

		if (bShouldBindToLocalHostAddr)
		{
			bool bCanBindAll = false;
			DoJoinMulticastGroup(MulticastAddr, ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->GetLocalHostAddr(*GLog, bCanBindAll), MulticastSocket);			
		}
	}
	else
	{
		TSharedRef<FInternetAddr> UnicastAddr = UnicastEndpoint.ToInternetAddr();
		DoJoinMulticastGroup(MulticastAddr, UnicastAddr , MulticastSocket);
	}
#endif
}

} // namespace UE::UdpMessaging::Private

bool FUdpMessageTransport::StartTransport(IMessageTransportHandler& Handler)
{
	const UUdpMessagingSettings* CurrentSettings = GetDefault<UUdpMessagingSettings>();

	// Set the handler even if initialization fails. This allows tries for reinitialization using the same handler.
	TransportHandler = &Handler;

#if PLATFORM_DESKTOP
	// create & initialize unicast socket (only on multi-process platforms)
	UnicastSocket = FUdpSocketBuilder(TEXT("UdpMessageUnicastSocket"))
		.AsNonBlocking()
		.BoundToEndpoint(UnicastEndpoint)
		.WithMulticastLoopback()
		.WithMulticastTtl(MulticastTtl) // since this socket is also used to send to multicast addresses
		.WithReceiveBufferSize(UDP_MESSAGING_RECEIVE_BUFFER_SIZE);

	if (UnicastSocket == nullptr)
	{
		UE_LOGF(LogUdpMessaging, Error, "StartTransport failed to create unicast socket on %ls", *UnicastEndpoint.ToString());
		return false;
	}
	else
	{
		int32 PortNo = UnicastSocket->GetPortNo();
		UE_LOGF(LogUdpMessaging, Display, "Unicast socket bound to '%ls:%d'.", *UnicastEndpoint.Address.ToString(), PortNo);
	}
#endif

	// create & initialize multicast socket (optional)
	MulticastSocket = FUdpSocketBuilder(TEXT("UdpMessageMulticastSocket"))
		.AsNonBlocking()
		.AsReusable()
#if PLATFORM_WINDOWS
		// If multiple bus instances bind the same unicast ip:port combination (allowed as the socket is marked as reusable), 
		// then for each packet sent to that ip:port combination, only one of the instances (at the discretion of the OS) will receive it. 
		// The instance that receives the packet may vary over time, seemingly based on the congestion of its socket.
		// This isn't the intended usage.
		// To allow traffic to be sent directly to unicast for discovery, set the interface and port for the unicast endpoint
		// However for legacy reason, keep binding this as well, although it might be unreliable in some cases. 
		.BoundToAddress(UnicastEndpoint.Address)
#endif
		.BoundToPort(MulticastEndpoint.Port)
#if PLATFORM_SUPPORTS_UDP_MULTICAST_GROUP
		.WithMulticastLoopback()
		.WithMulticastTtl(MulticastTtl)
		.WithMulticastInterface(UnicastEndpoint.Address)
#endif
		.WithReceiveBufferSize(UDP_MESSAGING_RECEIVE_BUFFER_SIZE);

	if (MulticastSocket != nullptr)
	{
		UE::UdpMessaging::Private::JoinedToGroup(UnicastEndpoint, MulticastEndpoint, MulticastSocket);
	}
	else
	{
		UE_LOGF(LogUdpMessaging, Warning, "StartTransport failed to create multicast socket on %ls, joined to %ls with TTL %i", *UnicastEndpoint.ToString(), *MulticastEndpoint.ToString(), MulticastTtl);

#if !PLATFORM_DESKTOP
		return false;
#endif
	}

	
	// initialize threads
	FTimespan ThreadWaitTime = FTimespan::FromMilliseconds(100);

#if PLATFORM_DESKTOP
	MessageProcessor = new FUdpMessageProcessor(*UnicastSocket, FGuid::NewGuid(), MulticastEndpoint);
#else
	check(MulticastSocket);
	MessageProcessor = new FUdpMessageProcessor(*MulticastSocket, FGuid::NewGuid(), MulticastEndpoint);
#endif
	{
		if (CurrentSettings->MaxConcurrentDeserializationTasks > 1)
		{
			UE_LOGF(LogUdpMessaging, Display, "Using asynchronous task graph for message deserialization.");
			DeserializationTasks = MakeUnique<UE::Tasks::FTaskConcurrencyLimiter>(CurrentSettings->MaxConcurrentDeserializationTasks);
		}
		
		// Add the static endpoints
		for (const FIPv4Endpoint& Endpoint : StaticEndpoints)
		{
			MessageProcessor->AddStaticEndpoint(Endpoint);
		}

		MessageProcessor->OnMessageReassembled().BindRaw(this, &FUdpMessageTransport::HandleProcessorMessageReassembled);
		MessageProcessor->OnNodeDiscovered().BindRaw(this, &FUdpMessageTransport::HandleProcessorNodeDiscovered);
		MessageProcessor->OnNodeLost().BindRaw(this, &FUdpMessageTransport::HandleProcessorNodeLost);
		MessageProcessor->OnError().BindRaw(this, &FUdpMessageTransport::HandleProcessorError);
		MessageProcessor->OnErrorSendingToEndpoint_UdpMessageProcessorThread().BindRaw(this, &FUdpMessageTransport::HandleEndpointCommunicationError);
		MessageProcessor->OnCanAcceptEndpoint_UdpMessageProcessorThread().BindRaw(this, &FUdpMessageTransport::HandleProcessorEndpointCheck);
		MessageProcessor->SetShareKnownNodesState(CurrentSettings->bShareKnownNodesWithActiveConnections);
	}

	if (MulticastSocket != nullptr)
	{
		MulticastReceiver = new FUdpSocketReceiver(MulticastSocket, ThreadWaitTime, TEXT("UdpMessageMulticastReceiver"));
		MulticastReceiver->OnDataReceived().BindRaw(this, &FUdpMessageTransport::HandleMulticastSocketDataReceived);
		MulticastReceiver->SetMaxReadBufferSize(65535);
		MulticastReceiver->Start();
	}
	else
	{
		UE_LOGF(LogUdpMessaging, Warning, "Multicast socket was null so there is no multicast receiver.");
	}

#if PLATFORM_DESKTOP
	UnicastReceiver = new FUdpSocketReceiver(UnicastSocket, ThreadWaitTime, TEXT("UdpMessageUnicastReceiver"));
	{
		UnicastReceiver->OnDataReceived().BindRaw(this, &FUdpMessageTransport::HandleSocketDataReceived);
		UnicastReceiver->SetMaxReadBufferSize(65535);
		UnicastReceiver->Start();
	}
	MessageLogger = MakeUnique<FUdpMessageProcessorLogger>(UnicastEndpoint, UnicastSocket ? UnicastSocket->GetPortNo() : 0, MessageProcessor);
#endif

	UE_LOGF(LogUdpMessaging, Verbose, "Started Transport");
	return true;
}


void FUdpMessageTransport::StopTransport()
{
	StopAutoRepairRoutine();

	// shut down threads
	delete MulticastReceiver;
	MulticastReceiver = nullptr;

#if PLATFORM_DESKTOP
	delete UnicastReceiver;
	UnicastReceiver = nullptr;
#endif

	delete MessageProcessor;
	MessageProcessor = nullptr;

	// destroy sockets
	if (MulticastSocket != nullptr)
	{
		SocketSubsystem->DestroySocket(MulticastSocket);
		MulticastSocket = nullptr;
	}
	
#if PLATFORM_DESKTOP
	if (UnicastSocket != nullptr)
	{
		SocketSubsystem->DestroySocket(UnicastSocket);
		UnicastSocket = nullptr;
	}
#endif

	// Clear out any deserialize tasks before resetting the transport handler. They depend on the handler to dispatch
	// deserialized messages.
	//
	if (DeserializationTasks)
	{
		DeserializationTasks->Wait();
		DeserializationTasks.Reset();		
	}
	
	TransportHandler = nullptr;
	ErrorFuture.Reset();
	UE_LOGF(LogUdpMessaging, Verbose, "Stopped Transport");
}


bool FUdpMessageTransport::TransportMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TArray<FGuid>& Recipients)
{
	if (MessageProcessor == nullptr)
	{
		return false;
	}

	if (Context->GetRecipients().Num() > UDP_MESSAGING_MAX_RECIPIENTS)
	{
		return false;
	}

#if !NO_LOGGING
	if (UE_GET_LOG_VERBOSITY(LogUdpMessaging) >= ELogVerbosity::Verbose)
	{
		FString RecipientStr = FString::JoinBy(Recipients, TEXT("+"), [](const FGuid& Guid) { return Guid.ToString(); });
		UE_LOGF(LogUdpMessaging, Verbose, "TransportMessage %ls from %ls to %ls", *Context->GetMessageTypePathName().ToString(), *Context->GetSender().ToString(), *RecipientStr);
	}
#endif

	return MessageProcessor->EnqueueOutboundMessage(Context, Recipients);
}


/* FUdpMessageTransport event handlers
 *****************************************************************************/

void FUdpMessageTransport::HandleProcessorMessageReassembled(FUdpReassembledMessageSharedPtr ReassembledMessagePtr, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const FGuid& NodeId)
{
	TSharedRef<FUdpDeserializedMessage, ESPMode::ThreadSafe> DeserializedMessage = MakeShared<FUdpDeserializedMessage, ESPMode::ThreadSafe>(Attachment);

	auto DeserializeReassembledMessage = [DeserializedMessage = MoveTemp(DeserializedMessage), TransportHandler = this->TransportHandler, ReassembledMessagePtr = MoveTemp(ReassembledMessagePtr), NodeId](uint32 Slot)
	{
		const FUdpReassembledMessage& ReassembledMessage = *ReassembledMessagePtr.Get();

		TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
			FPlatformTime::Cycles64(),
			ReassembledMessage.GetTraceMetadata().SenderShortId,
			ReassembledMessage.GetTraceMetadata().RecipientShortId,
			ReassembledMessage.GetTraceMetadata().MessageId,
			UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::BeginDeserialize
		);

		if (DeserializedMessage->Deserialize(ReassembledMessage))
		{
			TransportHandler->ReceiveTransportMessage(DeserializedMessage, NodeId);
		}
		else
		{
			TWeakObjectPtr<UScriptStruct> TypeInfo = ReassembledMessagePtr->GetMessageTypeInfo();
			FString TypeInfoStr = TypeInfo.IsValid() ? TypeInfo->GetName() : TEXT("UNKNOWN TYPE");
			UE_LOGF(LogUdpMessaging, Verbose, "Failed to deserialize message of type %ls from %ls", *TypeInfoStr, *NodeId.ToString());
		}

		// We always mark the message as delivered even if we can't deserialize it. There may be some messages that can't be deserialized because of missing type information.
		ReassembledMessagePtr->MarkDelivered();

		TRACE_UDPMESSAGING_MESSAGE_LIFECYCLE(
			FPlatformTime::Cycles64(),
			ReassembledMessage.GetTraceMetadata().SenderShortId,
			ReassembledMessage.GetTraceMetadata().RecipientShortId,
			ReassembledMessage.GetTraceMetadata().MessageId,
			UE::UdpMessaging::Trace::EUdpMessageLifecycleEvent::Delivered
		);
	};

	if (DeserializationTasks)
	{
		// The user has asked for async deserialization.  Dispatch the task.
		DeserializationTasks->Push(UE_SOURCE_LOCATION, MoveTemp(DeserializeReassembledMessage));
	}
	else
	{
		// Otherwise we should run on this thread. 
		DeserializeReassembledMessage(0);
	}
}


void FUdpMessageTransport::HandleProcessorNodeDiscovered(const FGuid& DiscoveredNodeId)
{
	TransportHandler->DiscoverTransportNode(DiscoveredNodeId);
}


void FUdpMessageTransport::HandleProcessorNodeLost(const FGuid& LostNodeId)
{
	TransportHandler->ForgetTransportNode(LostNodeId);
}

namespace UE::UdpMessageTransport::Private
{
	TArray<FString> TokenizeStringAddress(const FString &Address)
	{
		TArray<FString> Tokens;
		bool bIsValid = Address.ParseIntoArray(Tokens, TEXT("."), false /*CullEmpty*/) == 4;
		if (bIsValid)
		{
			for (const FString& Token : Tokens)
			{
				if (!FCString::IsNumeric(*Token))
				{
					// If it's not a pure number, look for individual characters
					for (int32 Index = 0; Index < Token.Len(); ++Index)
					{
						const TCHAR& Character = Token[Index];
						if (!(FChar::IsDigit(Character) || Character == '*' || Character == '?'))
						{
							bIsValid = false;
							break;
						}
					}

					if (!bIsValid)
					{
						break;
					}
				}
			}
		}
		return Tokens;
	}

	bool MatchesIPv4Address(const FString& SourceAddress, const FIPv4Address& TargetAddress)
	{
		FString TargetAsString = TargetAddress.ToString();
		TArray<FString> SourceTokenized = TokenizeStringAddress(SourceAddress);
		TArray<FString> TargetTokenized =  TokenizeStringAddress(TargetAsString);
		if (SourceTokenized.Num() == 4 && SourceTokenized.Num() == TargetTokenized.Num())
		{
			for (int32 TokenIndex = 0; TokenIndex < 4; ++TokenIndex)
			{
				const FString& Source = SourceTokenized[TokenIndex];
				const FString& Target = TargetTokenized[TokenIndex];
				if (!Target.MatchesWildcard(Source))
				{
					return false;
				}
			}

			// It must have passed all tokens.
			return true;
		}
		return true;
	}

	int32 GetPortNumber(const FString& InPortAsString)
	{
		if (InPortAsString.IsNumeric())
		{
			return FCString::Atoi(*InPortAsString);
		}
		return -1;
	}


}

bool FUdpMessageTransport::MatchesEndpoint(const FString& SourceEndpoint, const FIPv4Endpoint& TargetEndpoint)
{
	using namespace UE::UdpMessageTransport::Private;
	if (SourceEndpoint.IsEmpty())
	{
		return false;
	}

	TArray<FString> IpPortTokens;
	const int32 Items = SourceEndpoint.ParseIntoArray(IpPortTokens, TEXT(":"));
	if (Items <= 0)
	{
		return false;
	}

	const int32 Port = Items == 1 ? 0 : GetPortNumber(IpPortTokens[1]);
	return (Port == 0 || Port == TargetEndpoint.Port)
		&& MatchesIPv4Address(IpPortTokens[0], TargetEndpoint.Address);
}

bool FUdpMessageTransport::HandleProcessorEndpointCheck(const FGuid& EndpointNodeId, const FIPv4Endpoint& SenderEndpoint)
{
	if (CVarEndpointDenyListEnabled.GetValueOnAnyThread())
	{
		if (!UE::UdpMessageTransport::Private::GlobalDenyCandidateTable.ShouldAllowEndpoint(EndpointNodeId))
		{
			return false;
		}
	}

	for (const FString& Endpoint : ExcludedEndpoints)
	{
		if (MatchesEndpoint(Endpoint, SenderEndpoint))
		{
			return false;
		}
	}
	return true;
}

void FUdpMessageTransport::HandleEndpointCommunicationError(const FGuid& EndpointId, const FIPv4Endpoint& /*Unused EndpointIdAddress*/)
{
	UE::UdpMessageTransport::Private::GlobalDenyCandidateTable.HandleEndpointError(EndpointId);
}

void FUdpMessageTransport::HandleProcessorError()
{
	if (!ErrorFuture.IsValid()) {
		// Capture a weak pointer to this transport in the lambda to be executed later, and
		// try to pin it again when the function actually runs. This guards against the transport
		// being deleted in between the async task being scheduled and when it starts running.
		TWeakPtr<FUdpMessageTransport, ESPMode::ThreadSafe> WeakTransportPtr = AsShared();
		ErrorFuture = Async(EAsyncExecution::TaskGraphMainThread, [WeakTransport = WeakTransportPtr]()
		{
			// Bail out early if the UObject system is not initialized (e.g. at shutdown), since we
			// won't be able to access the settings CDO even if the transport still exists.
			if (!UObjectInitialized())
			{
				return;
			}

			if (TSharedPtr<FUdpMessageTransport, ESPMode::ThreadSafe> Transport = WeakTransport.Pin())
			{
				const UUdpMessagingSettings* Settings = GetDefault<UUdpMessagingSettings>();
				if (Settings->bAutoRepair)
				{
					Transport->StartAutoRepairRoutine(Settings->AutoRepairAttemptLimit);
				}
				else
				{
					UE_LOGF(LogUdpMessaging, Error, "UDP messaging encountered an error. Please restart the service for proper functionality");
				}
			}
		});
	}
}


void FUdpMessageTransport::StartAutoRepairRoutine(uint32 MaxRetryAttempt)
{
	StopAutoRepairRoutine();

	TWeakPtr<FUdpMessageTransport, ESPMode::ThreadSafe> WeakTransportPtr = AsShared();
	FTimespan CheckDelay(0, 0, 1);
	uint32 CheckNumber = 1;
	AutoRepairHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([WeakTransport = WeakTransportPtr, LastTime = FDateTime::UtcNow(), CheckDelay, CheckNumber, MaxRetryAttempt](float DeltaTime) mutable
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FUdpMessageTransport_AutoRepair);
		bool bContinue = true;
		FDateTime UtcNow = FDateTime::UtcNow();
		if (LastTime + (CheckDelay * CheckNumber) <= UtcNow)
		{
			if (auto Transport = WeakTransport.Pin())
			{
				// if the restart fail, continue the routine if we are still under the retry attempt limit
				bContinue = !Transport->RestartTransport();
				bContinue = bContinue && CheckNumber <= MaxRetryAttempt;
			}
			// if we do not have a valid transport also stop the routine
			else
			{
				bContinue = false;
			}
			++CheckNumber;
			LastTime = UtcNow;
		}
		return bContinue;
	}), 1.0f);
	UE_LOGF(LogUdpMessaging, Warning, "UDP messaging encountered an error. Auto repair routine started for reinitialization");
}


void FUdpMessageTransport::StopAutoRepairRoutine()
{
	if (AutoRepairHandle.IsValid())
	{
		FTSTicker::RemoveTicker(AutoRepairHandle);
	}
}


bool FUdpMessageTransport::RestartTransport()
{
	IMessageTransportHandler* Handler = TransportHandler;
	StopTransport();
	return StartTransport(*Handler);
}

void FUdpMessageTransport::HandleMulticastSocketDataReceived(const TSharedPtr<FArrayReader, ESPMode::ThreadSafe>& Data, const FIPv4Endpoint& Sender)
{
	if (MessageProcessor != nullptr)
	{
		MessageProcessor->EnqueueInboundSegment(Data, Sender);
	}
}


void FUdpMessageTransport::HandleSocketDataReceived(const TSharedPtr<FArrayReader, ESPMode::ThreadSafe>& Data, const FIPv4Endpoint& Sender)
{
	if (MessageProcessor != nullptr)
	{
		MessageProcessor->EnqueueInboundSegment(Data, Sender);
	}
}
