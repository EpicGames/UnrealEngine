// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageTransport.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"

#include "Async/Future.h"
#include "Containers/Ticker.h"
#include "Misc/Guid.h"
#include "Templates/UniquePtr.h"
#include "Tasks/TaskConcurrencyLimiter.h"

struct FMessageTransportStatistics;

class FArrayReader;
class FUdpReassembledMessage;
class FRunnableThread;
class FSocket;
class FUdpMessageProcessor;
class FUdpSocketReceiver;
class IMessageAttachment;
class IMessageContext;
class IMessageTransportHandler;
class ISocketSubsystem;
struct FUdpMessageProcessorLogger;

struct FGuid;

using FUdpReassembledMessageSharedPtr = TSharedPtr<FUdpReassembledMessage, ESPMode::ThreadSafe>;

/**
 * Implements a message transport technology using an UDP network connection.
 *
 * On platforms that support multiple processes, the transport is using two sockets,
 * one for per-process unicast sending/receiving, and one for multicast receiving.
 * Console and mobile platforms use a single multicast socket for all sending and receiving.
 */
class FUdpMessageTransport
	: public TSharedFromThis<FUdpMessageTransport, ESPMode::ThreadSafe>
	, public IMessageTransport
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InLocalEndpoint The local IP endpoint to receive messages on.
	 * @param InMulticastEndpoint The multicast group endpoint to transport messages to.
	 * @param InStaticEndpoints array of endpoints to include in communication
	 * @param InExcludedEndpoints array of string formatted endpoints to exclude in communication.
	 * @param InMulticastTtl The multicast time-to-live.
	 */
	FUdpMessageTransport(const FIPv4Endpoint& InLocalEndpoint,
						 const FIPv4Endpoint& InMulticastEndpoint,
						 TArray<FIPv4Endpoint> InStaticEndpoints,
						 TArray<FString> InExcludedEndpointsAsString,
						 uint8 InMulticastTtl);

	/** Virtual destructor. */
	virtual ~FUdpMessageTransport();

	/** Notifies the transport that the application is about to exit. */
	void OnAppPreExit();

	/**
	 *  Helper method to detect if a given target endpoint matches the wildcard deliminted source endpoint. The source endpoint should be in
	 *  IPv4 format.  A.B.C.D:<OptionalPortNumber>.  Wildcard characters are allowed.
	 */
	static bool MatchesEndpoint(const FString& SourceEndpoint, const FIPv4Endpoint& TargetEndpoint);

public:
	/**
	 * Add a static endpoint to the transport
	 * @param InEndpoint the endpoint to add
	 */
	void AddStaticEndpoint(const FIPv4Endpoint& InEndpoint);

	/**
	 * Remove a static endpoint from the transport
	 * @param InEndpoint the endpoint to remove
	 */
	void RemoveStaticEndpoint(const FIPv4Endpoint& InEndpoint);

	//~ IMessageTransport interface
	virtual FName GetDebugName() const override;
	virtual bool StartTransport(IMessageTransportHandler& Handler) override;
	virtual void StopTransport() override;
	virtual bool TransportMessage(const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& Context, const TArray<FGuid>& Recipients) override;

	/**
	 * Get the IPv4Endpoint listening addresses.
	 */
	TArray<FIPv4Endpoint> GetListeningAddresses() const;

	/**
	 * Get the known endpoints that are in active communication with the UDP transport. This is information
	 * known by the processor.
	 */
	TArray<FIPv4Endpoint> GetKnownEndpoints() const;

	/**
	 * Retrieve the latest network statistics for the given node.
	 */
	FMessageTransportStatistics GetLatestStatistics(FGuid NodeId) const;

private:

	/** Handles received transport messages. */
	void HandleProcessorMessageReassembled(FUdpReassembledMessageSharedPtr ReassembledMessagePtr, const TSharedPtr<IMessageAttachment, ESPMode::ThreadSafe>& Attachment, const FGuid& NodeId);

	/** Handles discovered transport endpoints. */
	void HandleProcessorNodeDiscovered(const FGuid& DiscoveredNodeId);

	/** Handles lost transport endpoints. */
	void HandleProcessorNodeLost(const FGuid& LostNodeId);

	/** */
	void HandleProcessorError();

	/** Returns true if we can add this endpoint to the list of nodes we communicate with. */
	bool HandleProcessorEndpointCheck(const FGuid& EndpointNodeId, const FIPv4Endpoint& SenderIpAddress);

	/** Handle endpoint communication error. */
	void HandleEndpointCommunicationError(const FGuid& EndpointId, const FIPv4Endpoint& EndpointIdAddress);

	/** Launches a routine that attempts to restart the transport. */
	void StartAutoRepairRoutine(uint32 MaxRetryAttempt);

	/** Halts any currently running transport restart routine. */
	void StopAutoRepairRoutine();

	/**
	 * Restart the transport using the same Transport handler.
	 * @return true if the restart was successful
	 */
	bool RestartTransport();

	/** Handles received socket data. */
	void HandleSocketDataReceived(const TSharedPtr<FArrayReader, ESPMode::ThreadSafe>& Data, const FIPv4Endpoint& Sender);

	/** Handles received socket data for multicast receiver. */
	void HandleMulticastSocketDataReceived(const TSharedPtr<FArrayReader, ESPMode::ThreadSafe>& Data, const FIPv4Endpoint& Sender);
		
private:
	/** Clear the deny list */
	void DoClearDenyCandidateList();

	/** Holds any pending restart request. */
	TFuture<void> ErrorFuture;

	/** Holds the delegate handle for the auto repair routine. */
	FTSTicker::FDelegateHandle AutoRepairHandle;

	/** Holds the message processor. */
	FUdpMessageProcessor* MessageProcessor;

	/** Holds the multicast endpoint. */
	FIPv4Endpoint MulticastEndpoint;

	/** Holds the multicast socket receiver. */
	FUdpSocketReceiver* MulticastReceiver;

	/** Holds the multicast socket. */
	FSocket* MulticastSocket;

	/** Holds the multicast time to live. */
	uint8 MulticastTtl;

	/** Holds a pointer to the socket sub-system. */
	ISocketSubsystem* SocketSubsystem;

	/** Message transport handler. */
	IMessageTransportHandler* TransportHandler;

	/** Holds the local endpoint to receive messages on. */
	FIPv4Endpoint UnicastEndpoint;

#if PLATFORM_DESKTOP
	/** Holds the unicast socket receiver. */
	FUdpSocketReceiver* UnicastReceiver;

	/** Holds the unicast socket. */
	FSocket* UnicastSocket;
#endif

	/** Holds the static endpoints. */
	TSet<FIPv4Endpoint> StaticEndpoints;

	/** Excluded list of IP addresses as string value (can contain wildcards) not allowed to talk to UDP processor */
	TArray<FString> ExcludedEndpoints;

	/** Async task queue for running deserialization of reassembled messages. */
	TUniquePtr<UE::Tasks::FTaskConcurrencyLimiter> DeserializationTasks;

#if PLATFORM_DESKTOP
	/** Logging class to enable inspection of message data. */
	TUniquePtr<FUdpMessageProcessorLogger> MessageLogger;
#endif
};
