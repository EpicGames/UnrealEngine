// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IMessageContext.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Shared/UdpMessagingTime.h"
#include "Trace/UdpMessagingTrace.h"
#include "UdpMessagingPrivate.h"

// IMessageContext forward declaration
enum class EMessageFlags : uint32;

/**
 * Implements a reassembled message.
 */
class FUdpReassembledMessage
{
public:
	enum class EDeliveryStatus : uint8
	{
		NotDelivered = 0,
		PendingDelivery,
		Delivered
	};

	/**
	 * Creates and initializes a new inbound message info.
	 *
	 * @param ProtocolVersion The protocol version the message is serialized in.
	 * @param MessageFlags The message flags of the reassembled message.
	 * @param MessageSize The total size of the message in bytes.
	 * @param SegmentCount The total number of segments that need to be received for this message.
	 * @param InSequence The message sequence number.
	 * @param InSender The IPv4 endpoint of the sender.
	 */
	FUdpReassembledMessage(uint8 InProtocolVersion, EMessageFlags InFlags, int64 MessageSize, uint32 SegmentCount, uint64 InSequence, const FIPv4Endpoint& InSender)
		: ProtocolVersion(InProtocolVersion)
		, MessageFlags(InFlags)
		, PendingSegmentsCount(SegmentCount)
		, ReceivedBytes(0)
		, DeliveryStatus(EDeliveryStatus::NotDelivered)
		, RetransmitRequestsCount(0)
		, Sender(InSender)
		, Sequence(InSequence)
	{
		// Containers only use int32 for indices, so we can't actually allow sizes greater than that
		// without causing overflow issues.
		if (MessageSize > static_cast<int64>(TNumericLimits<int32>::Max()) || MessageSize <= 0)
		{
			UE_LOGF(LogUdpMessaging, Warning, "FUdpReassembledMessage: Invalid MessageSize, marking message malformed. Message=%ls, MessageSize=%lli",
				*Describe(), MessageSize);
			bIsMalformed = true;
		}
		if (SegmentCount > static_cast<uint32>(TNumericLimits<int32>::Max()) || SegmentCount == 0)
		{
			UE_LOGF(LogUdpMessaging, Warning, "FUdpReassembledMessage: Invalid SegmentCount, marking message malformed. Message=%ls, SegmentCount=%u",
				*Describe(), SegmentCount);
			bIsMalformed = true;
		}

		// It's completely possible that MessageSize and SegmentCount are still incorrect due to corruption.
		// However, we have no way to detect that at this point, so we'll assume they're fine and validate
		// as we receive segments later.
		if (!IsMalformed())
		{
			PendingSegments = TBitArray<>(true, SegmentCount);
			Data.AddUninitialized(MessageSize);
		}
	}

	~FUdpReassembledMessage() = default;

public:

	/**
	 * Gets the message protocol version.
	 *
	 * @return The Message protocol version.
	 */
	uint8 GetProtocolVersion() const
	{
		return ProtocolVersion;
	}

	/**
	 * Gets the message flags
	 *
	 * @return The Message flags.
	 */
	EMessageFlags GetFlags() const
	{
		return MessageFlags;
	}

	/**
	 * Gets the message data.
	 *
	 * @return Message data.
	 */
	const TArray<uint8>& GetData() const
	{
		return Data;
	}

	/**
	 * Gets the time at which the last segment was received.
	 *
	 * @return Last receive time.
	 */
	FUdpMessagingTime GetLastSegmentTime() const
	{
		return LastSegmentTime;
	}

	/**
	 * Gets the list of segments that haven't been received yet.
	 *
	 * @return List of pending segment numbers.
	 */
	TArray<uint16> GetPendingSegments() const
	{
		TArray<uint16> Result;

		if (PendingSegmentsCount > 0)
		{
			for (TConstSetBitIterator<> It(PendingSegments); It; ++It)
			{
				Result.Add(It.GetIndex());
			}
		}

		return Result;
	}


	/**
	 * Gets the total number of segments.
	 *
	 * @return Number of total segments.
	 */
	uint32 GetTotalSegmentsCount() const
	{
		return PendingSegments.Num();
	}

	/**
	 * Gets the number of segments that haven't been received yet.
	 *
	 * @return Number of pending segments.
	 */
	uint32 GetPendingSegmentsCount() const
	{
		return PendingSegmentsCount;
	}

	/**
	 * Gets the number of retransmit requests that were sent for this payload.
	 *
	 * @return Number of retransmit requests.
	 */
	int32 GetRetransmitRequestsCount() const
	{
		return RetransmitRequestsCount;
	}

	/**
	 * Gets the message's sequence number.
	 *
	 * @return Sequence number.
	 */
	uint64 GetSequence() const
	{
		return Sequence;
	}

	/**
	 * Checks whether this message is complete.
	 *
	 * @return true if the message is complete, false otherwise.
	 */
	bool IsComplete() const
	{
		return (PendingSegmentsCount == 0);
	}

	/**
	 * Checks whether this message has been initialized.
	 *
	 * @return true if the message is initialized, false otherwise.
	 */
	bool IsInitialized() const
	{
		return (Data.Num() > 0);
	}

	/**
	 * @return How many acknowledgments are currently pending send.
	 */
	int32 NumPendingAcknowledgements() const
	{
		return PendingAcknowledgments.Num();
	}

	/**
	 * @return The time at which any acknowledgments were last transmitted for this message.
	 */
	FUdpMessagingTime GetLastAckTime() const
	{
		return LastAckTime;
	}

	/**
	 * Gets at most the specified number of pending acknowledgments and clears them, updating LastAckTime.
	 *
	 * @return The array of pending acknowledgments removed.
	 */
	TArray<uint32> TakePendingAcknowledgments(int32 MaxAckNum, FUdpMessagingTime CurrentTime)
	{
		check(MaxAckNum > 0);

		LastAckTime = CurrentTime;

		TArray<uint32> Temp;
		if (PendingAcknowledgments.Num() < MaxAckNum)
		{
			Swap(Temp, PendingAcknowledgments);
		}
		else
		{
			Temp.Append(PendingAcknowledgments.GetData(), MaxAckNum);
			PendingAcknowledgments.RemoveAt(0, MaxAckNum, EAllowShrinking::No);
		}
		return Temp;
	}

	/**
	 * Get if the message has been delivered.
	 * @return true if the message has been marked delivered.
	 */
	bool IsDelivered() const
	{
		return DeliveryStatus == EDeliveryStatus::Delivered;
	}

	/**
	 * Get if the message is pending delivery through the transport layer.
	 */
	bool IsPendingDelivery() const
	{
		return DeliveryStatus == EDeliveryStatus::PendingDelivery;
	}
		
	/**
	 * Mark the reasembled message as delivered.
	 */
	void MarkDelivered()
	{
		DeliveryStatus = EDeliveryStatus::Delivered;
	}

	/**
	 * Mark the reasembled message as pending delivery.
	 */
	void MarkPendingDelivery()
	{
		DeliveryStatus = EDeliveryStatus::PendingDelivery;
	}

	/**
	 * Indicates if the first segment of the data is ready.
	 **/
	bool HasFirstSegment() const
	{
		if (PendingSegments.Num() > 0)
		{
			return PendingSegments[0] == false;
		}
		return false;
	}

	/**
	 * Reassembles a segment into the specified message.
	 *
	 * @param SegmentNumber The number of the message segment.
	 * @param SegmentOffset The segment's offset within the message.
	 * @param SegmentData The segment data.
	 * @param CurrentTime The current time.
	 */
	void Reassemble(int32 SegmentNumber, int32 SegmentOffset, const TArray<uint8>& SegmentData, const FUdpMessagingTime CurrentTime)
	{
		if (IsPendingDelivery() || IsMalformed() || !IsInitialized() || SegmentData.Num() == 0)
		{
			return;
		}
		else if (!PendingSegments.IsValidIndex(SegmentNumber))
		{
			UE_LOGF(LogUdpMessaging, Warning, "FUdpReassembledMessage::Reassemble: Invalid Segment Number, ignoring segment. Message=%ls, Segment=%d",
				*Describe(), SegmentNumber);
			return;
		}

		if (PendingSegments[SegmentNumber])
		{
			const int32 MaxAllowableOffset = Data.Num() - SegmentData.Num();
			if (SegmentOffset < 0 || SegmentOffset > MaxAllowableOffset)
			{
				UE_LOGF(LogUdpMessaging, Warning, "FUdpReassembledMessage::Reassemble: SegmentOffset is invalid or would overflow Data. Message=%ls, Segment=%d, SegmentOffset=%d, SegmentSize=%d, DataSize=%d",
					*Describe(), SegmentNumber, SegmentOffset, SegmentData.Num(), Data.Num());
				return;
			}

			FMemory::Memcpy(Data.GetData() + SegmentOffset, SegmentData.GetData(), SegmentData.Num());

			PendingSegments[SegmentNumber] = false;
			--PendingSegmentsCount;
			ReceivedBytes += SegmentData.Num();
		}

		LastSegmentTime = CurrentTime;

		// Send acknowledgment for reliable messages
		if (EnumHasAnyFlags(GetFlags(), EMessageFlags::Reliable))
		{
			PendingAcknowledgments.AddUnique(SegmentNumber);
		}
	}

	/** True if this message is reliably delivered. */
	const bool IsReliable() const
	{
		return EnumHasAnyFlags(GetFlags(), EMessageFlags::Reliable);
	}

	/**
	 * Indicates whether or not this message was found to be malformed.
	 * Once we detect a message is malformed, we will not attempt to process any data for it.
	 */
	const bool IsMalformed() const
	{
		return bIsMalformed;
	}

	FString Describe() const
	{
		return FString::Printf(TEXT("Sender=%s, Sequence=%llu"), *Sender.ToString(), Sequence);
	}

	int32 GetReceivedBytes() const
	{
		return ReceivedBytes;
	}

	TWeakObjectPtr<UScriptStruct> GetMessageTypeInfo() const
	{
		return TypeInfo;
	}

	void SetMessageTypeInfo(TWeakObjectPtr<UScriptStruct> InTypeInfo)
	{
		TypeInfo = MoveTemp(InTypeInfo);
	}

#if UDPMESSAGINGTRACE_ENABLED
	struct FTraceMetadata
	{
		uint16 SenderShortId = 0;
		uint16 RecipientShortId = 0;
		int32 MessageId = 0;
	};

	const FTraceMetadata& GetTraceMetadata() const
	{
		return TraceMetadata;
	}

	FTraceMetadata& GetTraceMetadata()
	{
		return TraceMetadata;
	}
#endif

private:
	/** Holds the message's type information. */
	TWeakObjectPtr<UScriptStruct> TypeInfo;

	/** Holds the message protocol version. */
	uint8 ProtocolVersion;

	/** */
	EMessageFlags MessageFlags;

	/** Holds the message data. */
	TArray<uint8> Data;

	/** Holds the time at which the last segment was received. */
	FUdpMessagingTime LastSegmentTime;

	/** Holds an array of bits that indicate which segments still need to be received. */
	TBitArray<> PendingSegments;

	/** Holds the number of segments that haven't been received yet. */
	uint32 PendingSegmentsCount;

	/** Acknowledgment yet to be sent about segments we received */
	TArray<uint32> PendingAcknowledgments;

	/** Holds the time at which any pending acknowledgments were last transmitted. */
	FUdpMessagingTime LastAckTime;

	/** Holds the number of bytes received so far. */
	int32 ReceivedBytes;

	/** Holds if the reassembled message delivery status. This may get set from another thread which is why it is atomic. */
	std::atomic<EDeliveryStatus> DeliveryStatus;

	/** Holds the number of retransmit requests that were sent since the last segment was received. */
	int32 RetransmitRequestsCount;

	/** Holds the sender. */
	FIPv4Endpoint Sender;

	/** Holds the message sequence. */
	uint64 Sequence;

	/** Indicates whether or not this message is malformed. */
	bool bIsMalformed = false;

#if UDPMESSAGINGTRACE_ENABLED
	FTraceMetadata TraceMetadata;
#endif
};
