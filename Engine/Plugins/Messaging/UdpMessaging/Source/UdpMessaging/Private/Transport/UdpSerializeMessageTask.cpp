// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transport/UdpSerializeMessageTask.h"

#include "Backends/JsonStructSerializerBackend.h"
#include "Backends/CborStructSerializerBackend.h"
#include "Misc/DateTime.h"
#include "StructSerializer.h"

#include "UdpMessagingPrivate.h"
#include "Transport/UdpSerializedMessage.h"
#include "Shared/UdpMessagingSettings.h"
#include "UdpMessageSegmenter.h"
#include "Trace/UdpMessagingTrace.h"

namespace UdpSerializeMessageTaskDetails
{

/** Serialization Routine for message using Protocol version 10 */
void SerializeMessageV10(FArchive& Archive, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext)
{
	const FName& MessageType = MessageContext->GetMessageTypePathName().GetAssetName();
	Archive << const_cast<FName&>(MessageType);

	const FMessageAddress& Sender = MessageContext->GetSender();
	Archive << const_cast<FMessageAddress&>(Sender);

	const TArray<FMessageAddress>& Recipients = MessageContext->GetRecipients();
	Archive << const_cast<TArray<FMessageAddress>&>(Recipients);

	EMessageScope Scope = MessageContext->GetScope();
	Archive << Scope;

	const FDateTime& TimeSent = MessageContext->GetTimeSent();
	Archive << const_cast<FDateTime&>(TimeSent);

	const FDateTime& Expiration = MessageContext->GetExpiration();
	Archive << const_cast<FDateTime&>(Expiration);

	int32 NumAnnotations = MessageContext->GetAnnotations().Num();
	Archive << NumAnnotations;

	for (const auto& AnnotationPair : MessageContext->GetAnnotations())
	{
		Archive << const_cast<FName&>(AnnotationPair.Key);
		Archive << const_cast<FString&>(AnnotationPair.Value);
	}

	// serialize message body
	FJsonStructSerializerBackend Backend(Archive, EStructSerializerBackendFlags::None);
	FStructSerializer::Serialize(MessageContext->GetMessage(), *MessageContext->GetMessageTypeInfo(), Backend);
}

/** Serialization Routine for message using Protocol version 11, 12, 13, 14 or 15. */
void SerializeMessageV11_15(FArchive& Archive, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext, EUdpMessageFormat MessageFormat, const EStructSerializerBackendFlags StructSerializerBackendFlags)
{
	const FName& MessageType = MessageContext->GetMessageTypePathName().GetAssetName();
	Archive << const_cast<FName&>(MessageType);

	const FMessageAddress& Sender = MessageContext->GetSender();
	Archive << const_cast<FMessageAddress&>(Sender);

	const TArray<FMessageAddress>& Recipients = MessageContext->GetRecipients();
	Archive << const_cast<TArray<FMessageAddress>&>(Recipients);

	EMessageScope Scope = MessageContext->GetScope();
	Archive << Scope;

	EMessageFlags Flags = MessageContext->GetFlags();
	Archive << Flags;

	const FDateTime& TimeSent = MessageContext->GetTimeSent();
	Archive << const_cast<FDateTime&>(TimeSent);

	const FDateTime& Expiration = MessageContext->GetExpiration();
	Archive << const_cast<FDateTime&>(Expiration);

	int32 NumAnnotations = MessageContext->GetAnnotations().Num();
	Archive << NumAnnotations;

	for (const auto& AnnotationPair : MessageContext->GetAnnotations())
	{
		Archive << const_cast<FName&>(AnnotationPair.Key);
		Archive << const_cast<FString&>(AnnotationPair.Value);
	}

	// Message Wire Format Id
	check(MessageFormat == EUdpMessageFormat::CborPlatformEndianness || MessageFormat == EUdpMessageFormat::CborStandardEndianness); // Versions 11 to 15 only supports CBOR.
	uint8 Format = (uint8)(MessageFormat);
	Archive << Format;
	
	// serialize message body with cbor
	FCborStructSerializerBackend Backend(Archive, StructSerializerBackendFlags | (MessageFormat == EUdpMessageFormat::CborStandardEndianness ? EStructSerializerBackendFlags::WriteCborStandardEndianness : EStructSerializerBackendFlags::None));
	FStructSerializer::Serialize(MessageContext->GetMessage(), *MessageContext->GetMessageTypeInfo(), Backend);
}

/** Serialization Routine for message using Protocol version 16+. */
void SerializeMessageV16_18(FArchive& Archive, const TSharedRef<IMessageContext, ESPMode::ThreadSafe>& MessageContext, EUdpMessageFormat MessageFormat, const EStructSerializerBackendFlags StructSerializerBackendFlags)
{
	const FTopLevelAssetPath& MessageType = MessageContext->GetMessageTypePathName();
	Archive << const_cast<FTopLevelAssetPath&>(MessageType);

	const FMessageAddress& Sender = MessageContext->GetSender();
	Archive << const_cast<FMessageAddress&>(Sender);

	const TArray<FMessageAddress>& Recipients = MessageContext->GetRecipients();
	Archive << const_cast<TArray<FMessageAddress>&>(Recipients);

	EMessageScope Scope = MessageContext->GetScope();
	Archive << Scope;

	EMessageFlags Flags = MessageContext->GetFlags();
	Archive << Flags;

	const FDateTime& TimeSent = MessageContext->GetTimeSent();
	Archive << const_cast<FDateTime&>(TimeSent);

	const FDateTime& Expiration = MessageContext->GetExpiration();
	Archive << const_cast<FDateTime&>(Expiration);

	int32 NumAnnotations = MessageContext->GetAnnotations().Num();
	Archive << NumAnnotations;

	for (const auto& AnnotationPair : MessageContext->GetAnnotations())
	{
		Archive << const_cast<FName&>(AnnotationPair.Key);
		Archive << const_cast<FString&>(AnnotationPair.Value);
	}

	// Message Wire Format Id
	check(MessageFormat == EUdpMessageFormat::CborPlatformEndianness || MessageFormat == EUdpMessageFormat::CborStandardEndianness); // Versions 11 and up only support CBOR.
	uint8 Format = (uint8)(MessageFormat);
	Archive << Format;

	// serialize message body with cbor
	FCborStructSerializerBackend Backend(Archive, StructSerializerBackendFlags | (MessageFormat == EUdpMessageFormat::CborStandardEndianness ? EStructSerializerBackendFlags::WriteCborStandardEndianness : EStructSerializerBackendFlags::None));
	FStructSerializer::Serialize(MessageContext->GetMessage(), *MessageContext->GetMessageTypeInfo(), Backend);
}

} // namespace UdpSerializeMessageTaskDetails


/* FUdpSerializeMessageTask interface
 *****************************************************************************/

void FUdpSerializeMessageTask::DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent)
{
	SCOPED_MESSAGING_TRACE(FUdpSerializeMessageTask_DoTask);

#if UDPMESSAGINGTRACE_ENABLED
	FUdpSerializedMessage::FTraceMetadata& TraceMetadata = SerializedMessage->GetTraceMetadata();
	TraceMetadata.TypeInfo = MessageContext->GetMessageTypePathName();
	TraceMetadata.SerializationStartTime = FUdpMessagingTime::Now();
#endif

	if (MessageContext->IsValid())
	{
		// Note that some complex values are serialized manually here, so that we can ensure
		// a consistent wire format, if their implementations change. This allows us to sanity
		// check the values during deserialization. @see FUdpDeserializeMessage::Deserialize()

		const int64 MaxNumSegmentsV10_V11 = (int64)UINT16_MAX;
		const int64 MaxNumSegments = (int64)INT32_MAX;

		static_assert(UUdpMessagingSettings::MaxPacketSize_Min >
			(UE::UdpMessaging::PacketHeaderBytes + FUdpMessageSegmenter::GetDataSegmentHeaderSize()),
			"MaxPacketSize should always accommodate fixed headers");

		const int32 MaxSegmentSize = (int32)GetDefault<UUdpMessagingSettings>()->GetMaxPacketSize() - UE::UdpMessaging::PacketHeaderBytes;
		const uint16 MaxSegmentPayload = IntCastChecked<uint16>(MaxSegmentSize - FUdpMessageSegmenter::GetDataSegmentHeaderSize());

		// 2048 corresponds to old UnicastReceiver MaxReadBufferSize
		const int32 MaxSegmentSizeV10_V17 = FMath::Min(2048, MaxSegmentSize);
		const uint16 MaxSegmentPayloadV10_V17 = IntCastChecked<uint16>(MaxSegmentSizeV10_V17 - FUdpMessageSegmenter::GetDataSegmentHeaderSize());

		// serialize context depending on supported protocol version
		int64 ProtocolMaxMessageSize = MaxSegmentPayloadV10_V17 * MaxNumSegmentsV10_V11;
		FArchive& Archive = SerializedMessage.Get();
		bool Serialized = true;
		switch (SerializedMessage->GetProtocolVersion())
		{
			case 10:
				ProtocolMaxMessageSize = MaxSegmentPayloadV10_V17 * MaxNumSegmentsV10_V11;
				UdpSerializeMessageTaskDetails::SerializeMessageV10(Archive, MessageContext);
				break;

			case 11:
				ProtocolMaxMessageSize = MaxSegmentPayloadV10_V17 * MaxNumSegmentsV10_V11;
				UdpSerializeMessageTaskDetails::SerializeMessageV11_15(Archive, MessageContext, EUdpMessageFormat::CborPlatformEndianness, EStructSerializerBackendFlags::None);
				break;

			case 12:
				ProtocolMaxMessageSize = MaxSegmentPayloadV10_V17 * MaxNumSegments;
				UdpSerializeMessageTaskDetails::SerializeMessageV11_15(Archive, MessageContext, EUdpMessageFormat::CborPlatformEndianness, EStructSerializerBackendFlags::WriteTextAsComplexString);
				break;

			case 13:
				ProtocolMaxMessageSize = MaxSegmentPayloadV10_V17 * MaxNumSegments;
				UdpSerializeMessageTaskDetails::SerializeMessageV11_15(Archive, MessageContext, EUdpMessageFormat::CborPlatformEndianness, EStructSerializerBackendFlags::LegacyUE4);
				break;

			case 14:
				ProtocolMaxMessageSize = MaxSegmentPayloadV10_V17 * MaxNumSegments;
				UdpSerializeMessageTaskDetails::SerializeMessageV11_15(Archive, MessageContext, SerializedMessage->GetFormat(), EStructSerializerBackendFlags::LegacyUE4);
				break;

			case 15:
				ProtocolMaxMessageSize = MaxSegmentPayloadV10_V17 * MaxNumSegments;
				UdpSerializeMessageTaskDetails::SerializeMessageV11_15(Archive, MessageContext, SerializedMessage->GetFormat(), EStructSerializerBackendFlags::Default);
				break;

			case 16:
				ProtocolMaxMessageSize = MaxSegmentPayloadV10_V17 * MaxNumSegments;
				UdpSerializeMessageTaskDetails::SerializeMessageV16_18(Archive, MessageContext, SerializedMessage->GetFormat(), EStructSerializerBackendFlags::Default);
				break;

			case 17:
				ProtocolMaxMessageSize = MaxSegmentPayloadV10_V17 * MaxNumSegments;
				UdpSerializeMessageTaskDetails::SerializeMessageV16_18(Archive, MessageContext, SerializedMessage->GetFormat(), EStructSerializerBackendFlags::Default);
				break;

			case 18:
				ProtocolMaxMessageSize = MaxSegmentPayload * MaxNumSegments;
				UdpSerializeMessageTaskDetails::SerializeMessageV16_18(Archive, MessageContext, SerializedMessage->GetFormat(), EStructSerializerBackendFlags::Default);
				break;

			default:
				// Unsupported protocol version
				Serialized = false;
				break;
		}

		// if the message wasn't serialized, flag it invalid
		if (!Serialized)
		{
			UE_LOGF(LogUdpMessaging, Error, "Unsupported Protocol Version message tasked for serialization, discarding...");
			SerializedMessage->UpdateState(EUdpSerializedMessageState::Invalid);
		}
		// Once serialized if the size of the message is bigger than the maximum allow mark it as invalid and log an error
		else if (SerializedMessage->TotalSize() > ProtocolMaxMessageSize)
		{
			UE_LOG(LogUdpMessaging, Error, TEXT("Serialized Message total size '%" INT64_FMT "' is over the allowed maximum '%" INT64_FMT "', discarding..."), SerializedMessage->TotalSize(), ProtocolMaxMessageSize);
			SerializedMessage->UpdateState(EUdpSerializedMessageState::Invalid);
		}
		else
		{
			SerializedMessage->UpdateState(EUdpSerializedMessageState::Complete);
		}
	}
	else
	{
		SerializedMessage->UpdateState(EUdpSerializedMessageState::Invalid);
	}

#if UDPMESSAGINGTRACE_ENABLED
	TraceMetadata.SerializationEndTime = FUdpMessagingTime::Now();
#endif

	UE_LOG(LogUdpMessaging, Verbose, TEXT("Serialized %s from %s to %" INT64_FMT " bytes"), *MessageContext->GetMessageTypePathName().ToString(), *MessageContext->GetSender().ToString(), SerializedMessage->TotalSize());

	// signal task completion
	TSharedPtr<FEvent, ESPMode::ThreadSafe> CompletionEvent = CompletionEventPtr.Pin();

	if (CompletionEvent.IsValid())
	{
		CompletionEvent->Trigger();
	}
}


ENamedThreads::Type FUdpSerializeMessageTask::GetDesiredThread()
{
	return ENamedThreads::AnyThread;
}


TStatId FUdpSerializeMessageTask::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FUdpSerializeMessageTask, STATGROUP_TaskGraphTasks);
}


ESubsequentsMode::Type FUdpSerializeMessageTask::GetSubsequentsMode() 
{ 
	return ESubsequentsMode::FireAndForget; 
}
