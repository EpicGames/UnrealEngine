// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/TaskGraphInterfaces.h"

struct TStatId;

class FEvent;
class FUdpSerializedMessage;
class IMessageContext;


/**
 * Implements an asynchronous task for serializing a message.
 */
class FUdpSerializeMessageTask
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InMessageContext The context of the message to serialize.
	 * @param InSerializedMessage Will hold the serialized message data.
	 */
	FUdpSerializeMessageTask(TSharedRef<IMessageContext, ESPMode::ThreadSafe> InMessageContext, TSharedRef<FUdpSerializedMessage, ESPMode::ThreadSafe> InSerializedMessage, TSharedPtr<FEvent, ESPMode::ThreadSafe> InCompletionEvent)
		: CompletionEventPtr(InCompletionEvent)
		, MessageContext(InMessageContext)
		, SerializedMessage(InSerializedMessage)
	{ }

public:

	/**
	 * Performs the actual task.
	 *
	 * @param CurrentThread The thread that this task is executing on.
	 * @param MyCompletionGraphEvent The completion event.
	 */
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent);
	
	/**
	 * Returns the name of the thread that this task should run on.
	 *
	 * @return Thread name.
	 */
	ENamedThreads::Type GetDesiredThread();

	/**
	 * Gets the task's stats tracking identifier.
	 *
	 * @return Stats identifier.
	 */
	TStatId GetStatId() const;

	/**
	 * Gets the mode for tracking subsequent tasks.
	 *
	 * @return Always track subsequent tasks.
	 */
	static ESubsequentsMode::Type GetSubsequentsMode();

private:

	/** An event signaling that the message was serialized. */
	TWeakPtr<FEvent, ESPMode::ThreadSafe> CompletionEventPtr;

	/** The context of the message to serialize. */
	TSharedRef<IMessageContext, ESPMode::ThreadSafe> MessageContext;

	/** A reference to the serialized message data. */
	TSharedRef<FUdpSerializedMessage, ESPMode::ThreadSafe> SerializedMessage;
};
