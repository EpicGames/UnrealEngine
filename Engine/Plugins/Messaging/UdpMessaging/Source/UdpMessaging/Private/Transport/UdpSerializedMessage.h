// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "IMessageContext.h"
#include "Shared/UdpMessagingTime.h"
#include "Trace/UdpMessagingTrace.h"

#include <atomic>

enum class EUdpMessageFormat : uint8;

/**
 * Enumerates possible states of a serialized message.
 *
 * @see FUdpSerializedMessage
 */
enum class EUdpSerializedMessageState
{
	/** The message data is complete. */
	Complete,

	/** The message data is incomplete. */
	Incomplete,

	/** The message data is invalid. */
	Invalid
};


/**
 * Holds serialized message data.
 */
class FUdpSerializedMessage
	: public FMemoryWriter
{
public:
	/** Default constructor. */
	FUdpSerializedMessage(EUdpMessageFormat InMessageFormat, uint8 InProtocolVersion, EMessageFlags InFlags)
		: FMemoryWriter(DataArray, true)
		, State(EUdpSerializedMessageState::Incomplete)
		, Flags(InFlags)
		, Format(InMessageFormat)
		, ProtocolVersion(InProtocolVersion)
	{
		// Flags aren't supported in protocol version previous to 11
		if (ProtocolVersion < 11)
		{
			Flags = EMessageFlags::None;
		}
	}

public:

	/**
	 * Creates an archive reader to the data.
	 *
	 * The caller is responsible for deleting the returned object.
	 *
	 * @return An archive reader.
	 */
	FArchive* CreateReader()
	{
		return new FMemoryReader(DataArray, true);
	}

	/**
	 * Get the serialized message data.
	 *
	 * @return Byte array of message data.
	 * @see GetState
	 */
	const TArray<uint8>& GetDataArray()
	{
		return DataArray;
	}

	/**
	 * Gets the state of the message data.
	 *
	 * @return Message data state.
	 * @see GetData, UpdateState
	 */
	EUdpSerializedMessageState GetState() const
	{
		return State.load(std::memory_order_acquire);
	}

	/**
	 * Updates the state of this message data.
	 *
	 * @param InState The state to set.
	 * @see GetState
	 */
	void UpdateState(EUdpSerializedMessageState InState)
	{
		State.store(InState, std::memory_order_release);
	}

	/** @return the message flags. */
	EMessageFlags GetFlags() const
	{
		return Flags;
	}

	/** @return the message protocol version. */
	uint8 GetProtocolVersion() const
	{
		return ProtocolVersion;
	}

	/** @return the format used to encode the message. */
	EUdpMessageFormat GetFormat() const
	{
		return Format;
	}

#if UDPMESSAGINGTRACE_ENABLED
	struct FTraceMetadata
	{
		FTopLevelAssetPath TypeInfo;
		FUdpMessagingTime SerializationStartTime;
		FUdpMessagingTime SerializationEndTime;
	};

	const FTraceMetadata& GetTraceMetadata() const
	{
		return TraceMetadata;
	}

	FTraceMetadata& GetTraceMetadata()
	{
		return TraceMetadata;
	}
#endif

private:
	/** Holds the serialized data. */
	TArray<uint8> DataArray;

	/** Holds the message data state. */
	std::atomic<EUdpSerializedMessageState> State;

	/** Holds message flags, captured from context. */
	EMessageFlags Flags;

	/** The format used to serialize the message. */
	EUdpMessageFormat Format;

	/** Holds the Protocol Version the message will be serialized in. */
	uint8 ProtocolVersion;

#if UDPMESSAGINGTRACE_ENABLED
	FTraceMetadata TraceMetadata;
#endif
};
