// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tunnel/UdpMessageTunnel.h"
#include "UdpMessagingPrivate.h"

#include "Common/TcpListener.h"
#include "Common/UdpSocketBuilder.h"
#include "Serialization/ArrayReader.h"

#include "Shared/UdpMessageSegment.h"
#include "Tunnel/UdpMessageTunnelConnection.h"


#if PLATFORM_DESKTOP



/* FUdpMessageTunnel structors
 *****************************************************************************/

FUdpMessageTunnel::FUdpMessageTunnel(const FIPv4Endpoint& InUnicastEndpoint, const FIPv4Endpoint& InMulticastEndpoint)
	: Listener(nullptr)
	, MulticastEndpoint(InMulticastEndpoint)
	, Stopping(false)
	, Thread(nullptr)
	, TotalInboundBytes(0)
	, TotalOutboundBytes(0)
{
	UnicastSocket = FUdpSocketBuilder(TEXT("UdpMessageUnicastSocket"))
		.AsNonBlocking()
		.BoundToEndpoint(InUnicastEndpoint)
		.WithReceiveBufferSize(UDP_MESSAGING_RECEIVE_BUFFER_SIZE);

	if (UnicastSocket == nullptr)
	{
		UE_LOGF(LogUdpMessaging, Error, "UdpMessageTunnel failed to create unicast socket on %ls", *InUnicastEndpoint.ToString());
		Stopping = true;
	}

	// initialize sockets
	MulticastSocket = FUdpSocketBuilder(TEXT("UdpMessageMulticastSocket"))
		.AsNonBlocking()
		.AsReusable()
#if PLATFORM_WINDOWS
		// For 0.0.0.0, Windows will pick the default interface instead of all
		// interfaces. Here we allow to specify which interface to bind to. 
		// On all other platforms we bind to the wildcard IP address in order
		// to be able to also receive packets that were sent directly to the
		// interface IP instead of the multicast address.
		.BoundToAddress(InUnicastEndpoint.Address)
#endif
		.BoundToPort(InMulticastEndpoint.Port)
		.JoinedToGroup(InMulticastEndpoint.Address, InUnicastEndpoint.Address)
		.WithMulticastLoopback()
		.WithMulticastTtl(1)
		.WithMulticastInterface(InUnicastEndpoint.Address)
		.WithReceiveBufferSize(UDP_MESSAGING_RECEIVE_BUFFER_SIZE);

	if (MulticastSocket == nullptr)
	{
		UE_LOGF(LogUdpMessaging, Error, "UdpMessageTunnel failed to create multicast socket on %ls, joined to %ls", *InUnicastEndpoint.ToString(), *InMulticastEndpoint.ToString());
		Stopping = true;
	}

	// Start the tunnel only if properly initialized
	if (!Stopping)
	{
		Thread = FRunnableThread::Create(this, TEXT("FUdpMessageTunnel"), 128 * 1024, TPri_AboveNormal);
	}
}


FUdpMessageTunnel::~FUdpMessageTunnel()
{
	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
	}

	// destroy sockets
	if (MulticastSocket != nullptr)
	{
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(MulticastSocket);
		MulticastSocket = nullptr;
	}
	
	if (UnicastSocket != nullptr)
	{
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(UnicastSocket);
		UnicastSocket = nullptr;
	}
}


/* FRunnable interface
 *****************************************************************************/

FSingleThreadRunnable* FUdpMessageTunnel::GetSingleThreadInterface()
{
	return this;
}


bool FUdpMessageTunnel::Init()
{
	return true;
}


uint32 FUdpMessageTunnel::Run()
{
	while (!Stopping)
	{
		Update();
		FPlatformProcess::Sleep(0.0f);
	}

	return 0;
}


void FUdpMessageTunnel::Stop()
{
	Stopping = true;
}


void FUdpMessageTunnel::Exit()
{
	// do nothing
}


/* IUdpMessageTunnel interface
 *****************************************************************************/

bool FUdpMessageTunnel::Connect(const FIPv4Endpoint& RemoteEndpoint)
{
	FSocket* Socket = FTcpSocketBuilder(TEXT("FUdpMessageTunnel.RemoteConnection"));

	if (Socket == nullptr)
	{
		return false;
	}

	if (!Socket->Connect(*RemoteEndpoint.ToInternetAddr()))
	{
		ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->DestroySocket(Socket);

		return false;
	}

	PendingConnections.Enqueue(MakeShareable(new FUdpMessageTunnelConnection(Socket, RemoteEndpoint)));

	return true;
}


int32 FUdpMessageTunnel::GetConnections(TArray<TSharedPtr<IUdpMessageTunnelConnection>>& OutConnections)
{
	FScopeLock Lock(&CriticalSection);

	for (const auto& Connection : Connections)
	{
		OutConnections.Add(Connection);
	}

	return OutConnections.Num();
}


uint64 FUdpMessageTunnel::GetTotalInboundBytes() const
{
	return TotalInboundBytes;
}


uint64 FUdpMessageTunnel::GetTotalOutboundBytes() const
{
	return TotalOutboundBytes;
}


bool FUdpMessageTunnel::IsServerRunning() const
{
	return (Listener != nullptr);
}


FSimpleDelegate& FUdpMessageTunnel::OnConnectionsChanged()
{
	return ConnectionsChangedDelegate;
}


void FUdpMessageTunnel::StartServer(const FIPv4Endpoint& LocalEndpoint)
{
	StopServer();

	Listener = new FTcpListener(LocalEndpoint);
	Listener->OnConnectionAccepted().BindRaw(this, &FUdpMessageTunnel::HandleListenerConnectionAccepted);
}


void FUdpMessageTunnel::StopServer()
{
	delete Listener;
	Listener = nullptr;
}


/* FUdpMessageTunnel implementation
 *****************************************************************************/

void FUdpMessageTunnel::RemoveExpiredNodes(TMap<FGuid, FNodeInfo>& Nodes)
{
	for (TMap<FGuid, FNodeInfo>::TIterator It(Nodes); It; ++It)
	{
		if (CurrentTime - It.Value().LastDatagramReceivedTime > FTimespan::FromMinutes(2.0))
		{
			It.RemoveCurrent();
		}
	}
}


void FUdpMessageTunnel::TcpToUdp()
{
	TArray<TSharedPtr<FUdpMessageTunnelConnection>>::TIterator It(Connections);
	TSharedPtr<FArrayReader, ESPMode::ThreadSafe> Payload;

	while (It && UnicastSocket->Wait(ESocketWaitConditions::WaitForWrite, FTimespan::Zero()))
	{
		if ((*It)->Receive(Payload))
		{
			FUdpMessageSegment::FHeader Header;
			*Payload << Header;

			// check protocol version
			if (Header.ProtocolVersion > UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION)
			{
				return;
			}

			Payload->Seek(0);

			// update remote node & statistics
			FNodeInfo& RemoteNode = RemoteNodes.FindOrAdd(Header.SenderNodeId);
			RemoteNode.Connection = *It;
			RemoteNode.LastDatagramReceivedTime = CurrentTime;

			TotalInboundBytes += Payload->Num();

			// forward payload to local nodes
			FIPv4Endpoint RecipientEndpoint;

			if (Header.RecipientNodeId.IsValid())
			{
				FNodeInfo* LocalNode = LocalNodes.Find(Header.RecipientNodeId);

				if (LocalNode == nullptr)
				{
					continue;
				}

				RecipientEndpoint = LocalNode->Endpoint;
			}
			else
			{
				RecipientEndpoint = MulticastEndpoint;
			}

			int32 BytesSent = 0;
			UnicastSocket->SendTo(Payload->GetData(), Payload->Num(), BytesSent, *RecipientEndpoint.ToInternetAddr());
		}
		else
		{
			++It;
		}
	}
}


void FUdpMessageTunnel::UdpToTcp(FSocket* Socket)
{
	TSharedRef<FInternetAddr> Sender = ISocketSubsystem::Get(PLATFORM_SOCKETSUBSYSTEM)->CreateInternetAddr();

	uint32 DatagramSize = 0;

	while (Socket->HasPendingData(DatagramSize))
	{
		TSharedRef<FArrayReader, ESPMode::ThreadSafe> Datagram = MakeShareable(new FArrayReader(true));
		Datagram->SetNumUninitialized(FMath::Min(DatagramSize, 65507u));

		int32 BytesRead = 0;

		if (Socket->RecvFrom(Datagram->GetData(), Datagram->Num(), BytesRead, *Sender))
		{
			FUdpMessageSegment::FHeader Header;
			*Datagram << Header;

			// check protocol version
			if (Header.ProtocolVersion > UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION)
			{
				return;
			}

			// ignore loopback datagrams
			if (RemoteNodes.Contains(Header.SenderNodeId))
			{
				return;
			}

			Datagram->Seek(0);

			// forward datagram to remote nodes
			if (Header.RecipientNodeId.IsValid())
			{
				FNodeInfo* RemoteNode = RemoteNodes.Find(Header.RecipientNodeId);

				if ((RemoteNode != nullptr) && (RemoteNode->Connection.IsValid()))
				{
					RemoteNode->Connection->Send(Datagram);
				}
			}
			else
			{
				for (int32 ConnectionIndex = 0; ConnectionIndex < Connections.Num(); ++ConnectionIndex)
				{
					Connections[ConnectionIndex]->Send(Datagram);
				}
			}

			// update local node & statistics
			FNodeInfo& LocalNode = LocalNodes.FindOrAdd(Header.SenderNodeId);
			LocalNode.Endpoint = FIPv4Endpoint(Sender);
			LocalNode.LastDatagramReceivedTime = CurrentTime;

			TotalOutboundBytes += Datagram->Num();
		}
	}
}


void FUdpMessageTunnel::Update()
{
	CurrentTime = FDateTime::UtcNow();

	UpdateConnections();

	UdpToTcp(MulticastSocket);
	UdpToTcp(UnicastSocket);

	TcpToUdp();

	RemoveExpiredNodes(LocalNodes);
	RemoveExpiredNodes(RemoteNodes);
}


void FUdpMessageTunnel::UpdateConnections()
{
	bool ConnectionsChanged = false;

	// remove closed connections
	for (int32 ConnectionIndex = Connections.Num() - 1; ConnectionIndex >= 0; --ConnectionIndex)
	{
		if (!Connections[ConnectionIndex]->IsOpen())
		{
			Connections.RemoveAtSwap(ConnectionIndex);
			ConnectionsChanged = true;
		}
	}

	// add pending connections
	if (!PendingConnections.IsEmpty())
	{
		TSharedPtr<FUdpMessageTunnelConnection> PendingConnection;

		while (PendingConnections.Dequeue(PendingConnection))
		{
			Connections.Add(PendingConnection);
		}

		ConnectionsChanged = true;
	}

	// notify application
	if (ConnectionsChanged)
	{
		ConnectionsChangedDelegate.ExecuteIfBound();
	}
}


/* FSingleThreadRunnable interface
 *****************************************************************************/

void FUdpMessageTunnel::Tick()
{
	Update();
}


/* FUdpMessageTunnel callbacks
 *****************************************************************************/

bool FUdpMessageTunnel::HandleListenerConnectionAccepted(FSocket* ClientSocket, const FIPv4Endpoint& ClientEndpoint)
{
	PendingConnections.Enqueue(MakeShareable(new FUdpMessageTunnelConnection(ClientSocket, ClientEndpoint)));

	return true;
}


#endif
