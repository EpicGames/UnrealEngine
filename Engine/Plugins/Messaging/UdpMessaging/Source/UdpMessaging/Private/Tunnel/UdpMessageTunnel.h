// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/DateTime.h"
#include "Misc/Guid.h"
#include "HAL/Runnable.h"
#include "Containers/MpscQueue.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/SingleThreadRunnable.h"

#include "IUdpMessageTunnel.h"

class FSocket;
class FTcpListener;
class FUdpMessageTunnelConnection;


/**
 * Implements a bi-directional tunnel to send UDP messages over a TCP connection.
 */
class FUdpMessageTunnel
	: public FRunnable
	, public IUdpMessageTunnel
	, private FSingleThreadRunnable
{
	/** Structure for transport node information. */
	struct FNodeInfo
	{
		/** Holds the connection that owns the node (only used for remote nodes). */
		TSharedPtr<FUdpMessageTunnelConnection> Connection;

		/** Holds the node's IP endpoint (only used for local nodes). */
		FIPv4Endpoint Endpoint;

		/** Holds the time at which the last datagram was received. */
		FDateTime LastDatagramReceivedTime;
	};

public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InUnicastEndpoint The local IP endpoint to receive messages on.
	 * @param InMulticastEndpoint The multicast group endpoint to transport messages to.
	 */
	FUdpMessageTunnel(const FIPv4Endpoint& InUnicastEndpoint, const FIPv4Endpoint& InMulticastEndpoint);

	/** Virtual destructor. */
	virtual ~FUdpMessageTunnel();

public:

	//~ FRunnable interface

	virtual FSingleThreadRunnable* GetSingleThreadInterface() override;
	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;

public:

	//~ IUdpMessageTunnel interface

	virtual bool Connect(const FIPv4Endpoint& RemoteEndpoint) override;
	virtual int32 GetConnections(TArray<TSharedPtr<IUdpMessageTunnelConnection>>& OutConnections) override;
	virtual uint64 GetTotalInboundBytes() const override;
	virtual uint64 GetTotalOutboundBytes() const override;
	virtual bool IsServerRunning() const override;
	virtual FSimpleDelegate& OnConnectionsChanged() override;
	virtual void StartServer(const FIPv4Endpoint& LocalEndpoint) override;
	virtual void StopServer() override;

protected:

	/**
	 * Removes expired nodes from the specified collection of node infos.
	 *
	 * @param Nodes The collection of nodes to clean up.
	 */
	void RemoveExpiredNodes(TMap<FGuid, FNodeInfo>& Nodes);

	/**
	 * Receives all pending payloads from the tunnels and forwards them to the local message bus.
	 *
	 * @see UdpToTcp
	 */
	void TcpToUdp();

	/**
	 * Receives all buffered datagrams from the specified socket and forwards them to the tunnels.
	 *
	 * @param Socket The socket to receive from.
	 * @see TcpToUdp
	 */
	void UdpToTcp(FSocket* Socket);

	/** Update the tunnel. */
	void Update();

	/** Updates all active and pending connections. */
	void UpdateConnections();

protected:

	//~ FSingleThreadRunnable interface

	virtual void Tick() override;

private:

	/** Callback for accepted connections to the local tunnel server. */
	bool HandleListenerConnectionAccepted(FSocket* ClientSocket, const FIPv4Endpoint& ClientEndpoint);

private:

	/** Holds the list of open tunnel connections. */
	TArray<TSharedPtr<FUdpMessageTunnelConnection>> Connections;

	/** Holds a critical section for serializing access to the connections list. */
	FCriticalSection CriticalSection;

	/** Holds the current time. */
	FDateTime CurrentTime;

	/** Holds the local listener for incoming tunnel connections. */
	FTcpListener* Listener;

	/** Holds a collection of information for local transport nodes. */
	TMap<FGuid, FNodeInfo> LocalNodes;

	/** Holds the multicast endpoint. */
	FIPv4Endpoint MulticastEndpoint;

	/** Holds the multicast socket. */
	FSocket* MulticastSocket;

	/** Holds a queue of pending connections. */
	TMpscQueue<TSharedPtr<FUdpMessageTunnelConnection>> PendingConnections;

	/** Holds a collection of information for remote transport nodes. */
	TMap<FGuid, FNodeInfo> RemoteNodes;

	/** Holds a flag indicating that the thread is stopping. */
	bool Stopping;

	/** Holds the thread object. */
	FRunnableThread* Thread;

	/** Holds the total number of bytes that were received from tunnels. */
	uint64 TotalInboundBytes;

	/** Holds the total number of bytes that were sent out through tunnels. */
	uint64 TotalOutboundBytes;

	/** Holds the unicast socket. */
	FSocket* UnicastSocket;

private:

	/** Holds a delegate that is executed when the list of incoming connections changed. */
	FSimpleDelegate ConnectionsChangedDelegate;
};
