// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tunnel/UdpMessageTunnelConnection.h"

#include "HAL/PlatformProcess.h"
#include "HAL/RunnableThread.h"
#include "Serialization/ArrayReader.h"
#include "Serialization/ArrayWriter.h"
#include "Sockets.h"


#if PLATFORM_DESKTOP

/* FUdpMessageTunnelConnection structors
 *****************************************************************************/

FUdpMessageTunnelConnection::FUdpMessageTunnelConnection(FSocket* InSocket, const FIPv4Endpoint& InRemoteEndpoint)
	: OpenedTime(FDateTime::UtcNow())
	, RemoteEndpoint(InRemoteEndpoint)
	, Socket(InSocket)
	, TotalBytesReceived(0)
	, TotalBytesSent(0)
	, bRun(false)
{
	int32 NewSize = 0;
	Socket->SetReceiveBufferSize(2*1024*1024, NewSize);
	Socket->SetSendBufferSize(2*1024*1024, NewSize);
	Thread = FRunnableThread::Create(this, TEXT("FUdpMessageTunnelConnection"), 128 * 1024, TPri_Normal);
}


FUdpMessageTunnelConnection::~FUdpMessageTunnelConnection()
{
	if (Thread != nullptr)
	{
		Thread->Kill(true);
		delete Thread;
	}

	delete Socket;
	Socket = nullptr;
}


/* FUdpMessageTunnelConnection interface
 *****************************************************************************/

bool FUdpMessageTunnelConnection::Receive(TSharedPtr<FArrayReader, ESPMode::ThreadSafe>& OutPayload)
{
	return Inbox.Dequeue(OutPayload);
}


bool FUdpMessageTunnelConnection::Send(const TSharedPtr<FArrayReader, ESPMode::ThreadSafe>& Payload)
{
	if (IsOpen())
	{
		Outbox.Enqueue(Payload);
		return true;
	}

	return false;
}


/* FRunnable interface
 *****************************************************************************/

bool FUdpMessageTunnelConnection::Init()
{
	return (Socket != nullptr);
}


uint32 FUdpMessageTunnelConnection::Run()
{
	bRun = true;

	while (bRun)
	{
		if (Socket->Wait(ESocketWaitConditions::WaitForReadOrWrite, FTimespan::Zero()))
		{
			bRun = ReceivePayloads();
			bRun = SendPayloads();
		}

		FPlatformProcess::Sleep(0.0f);
	}

	ClosedTime = FDateTime::UtcNow();

	return 0;
}


void FUdpMessageTunnelConnection::Stop()
{
	Socket->Close();
	bRun = false;
}


void FUdpMessageTunnelConnection::Exit()
{
	// do nothing
}


/* IUdpMessageTunnelConnection interface
 *****************************************************************************/

void FUdpMessageTunnelConnection::Close()
{
	Stop();
}


uint64 FUdpMessageTunnelConnection::GetTotalBytesReceived() const
{
	return TotalBytesReceived;
}


uint64 FUdpMessageTunnelConnection::GetTotalBytesSent() const
{
	return TotalBytesSent;
}


FText FUdpMessageTunnelConnection::GetName() const
{
	return RemoteEndpoint.ToText();
}


FTimespan FUdpMessageTunnelConnection::GetUptime() const
{
	if (IsOpen())
	{
		return (FDateTime::UtcNow() - OpenedTime);
	}
		
	return (ClosedTime - OpenedTime);
}


bool FUdpMessageTunnelConnection::IsOpen() const
{
	return (Socket->GetConnectionState() == SCS_Connected);
}


/* FUdpMessageTunnelConnection implementation
 *****************************************************************************/

bool FUdpMessageTunnelConnection::ReceivePayloads()
{
	uint32 PendingDataSize = 0;

	// if we received a payload size...
	while (Socket->HasPendingData(PendingDataSize) && (PendingDataSize >= sizeof(uint16)))
	{
		int32 BytesRead = 0;
			
		FArrayReader PayloadSizeData = FArrayReader(true);
		PayloadSizeData.SetNumUninitialized(sizeof(uint16));
			
		// ... read it from the stream without removing it...
		if (!Socket->Recv(PayloadSizeData.GetData(), sizeof(uint16), BytesRead, ESocketReceiveFlags::Peek))
		{
			return false;
		}

		check(BytesRead == sizeof(uint16));

		uint16 PayloadSize;
		PayloadSizeData << PayloadSize;

		// ... and if we received the complete payload...
		if (Socket->HasPendingData(PendingDataSize) && (PendingDataSize >= sizeof(uint16) + PayloadSize))
		{
			// ... then remove the payload size from the stream...
			if (!Socket->Recv((uint8*)&PayloadSize, sizeof(uint16), BytesRead))
			{
				return false;
			}

			check(BytesRead == sizeof(uint16));
			TotalBytesReceived += BytesRead;

			// ... receive the payload
			TSharedPtr<FArrayReader, ESPMode::ThreadSafe> Payload = MakeShareable(new FArrayReader(true));
			Payload->SetNumUninitialized(PayloadSize);

			if (!Socket->Recv(Payload->GetData(), Payload->Num(), BytesRead))
			{
				return false;
			}

			check (BytesRead == PayloadSize);
			TotalBytesReceived += BytesRead;

			// .. and move it to the inbox
			Inbox.Enqueue(Payload);

			return true;
		}
	}

	return true;
}


bool FUdpMessageTunnelConnection::SendPayloads()
{
	if (Outbox.IsEmpty())
	{
		return true;
	}

	if (!Socket->Wait(ESocketWaitConditions::WaitForWrite, FTimespan::Zero()))
	{
		return true;
	}

	TSharedPtr<FArrayReader, ESPMode::ThreadSafe> Payload;
	Outbox.Dequeue(Payload);
	int32 BytesSent = 0;

	// send the payload size
	FArrayWriter PayloadSizeData = FArrayWriter(true);
	uint16 PayloadSize = Payload->Num();
	PayloadSizeData << PayloadSize;

	if (!Socket->Send(PayloadSizeData.GetData(), sizeof(uint16), BytesSent))
	{
		return false;
	}

	TotalBytesSent += BytesSent;

	// send the payload
	if (!Socket->Send(Payload->GetData(), Payload->Num(), BytesSent))
	{
		return false;
	}

	TotalBytesSent += BytesSent;

	return true;
}


#endif
