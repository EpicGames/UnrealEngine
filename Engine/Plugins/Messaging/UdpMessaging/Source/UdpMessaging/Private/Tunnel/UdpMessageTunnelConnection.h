// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SpscQueue.h"
#include "HAL/Runnable.h"
#include "Interfaces/IPv4/IPv4Endpoint.h"
#include "Misc/DateTime.h"

#include "IUdpMessageTunnelConnection.h"

class FArrayReader;
class FRunnableThread;
class FSocket;


/**
 * Implements a UDP message tunnel connection.
 */
class FUdpMessageTunnelConnection
	: public FRunnable
	, public TSharedFromThis<FUdpMessageTunnelConnection>
	, public IUdpMessageTunnelConnection
{
public:

	/**
	 * Creates and initializes a new instance.
	 *
	 * @param InSocket The socket to use for this connection.
	 * @param InRemoteAddress The IP endpoint of the remote client.
	 */
	FUdpMessageTunnelConnection(FSocket* InSocket, const FIPv4Endpoint& InRemoteEndpoint);

	/** Virtual destructor. */
	virtual ~FUdpMessageTunnelConnection();

public:

	/**
	 * Receives a payload from the connection's inbox.
	 *
	 * @param OutPayload Will hold the received payload, if any.
	 * @return true if a payload was returned, false otherwise.
	 * @see Send
	 */
	bool Receive(TSharedPtr<FArrayReader, ESPMode::ThreadSafe>& OutPayload);

	/**
	 * Sends a payload through this connection.
	 *
	 * @param Payload The payload to send.
	 * @see Receive
	 */
	bool Send(const TSharedPtr<FArrayReader, ESPMode::ThreadSafe>& Payload);

public:

	//~ FRunnable interface

	virtual bool Init() override;
	virtual uint32 Run() override;
	virtual void Stop() override;
	virtual void Exit() override;

public:

	//~ IUdpMessageTunnelConnection interface

	virtual void Close() override;
	virtual uint64 GetTotalBytesReceived() const override;
	virtual uint64 GetTotalBytesSent() const override;
	virtual FText GetName() const override;
	virtual FTimespan GetUptime() const override;
	virtual bool IsOpen() const override;

protected:

	/**
	 * Receives all pending payloads from the socket.
	 *
	 * @return true on success, false otherwise.
	 * @see SendPayloads
	 */
	bool ReceivePayloads();

	/**
	 * Sends all pending payloads to the socket.
	 *
	 * @return true on success, false otherwise.
	 * @see ReceivePayloads
	 */
	bool SendPayloads();

private:

	/** Holds the time at which the connection was closed. */
	FDateTime ClosedTime;

	/** Holds the collection of received payloads. */
	TSpscQueue<TSharedPtr<FArrayReader, ESPMode::ThreadSafe>> Inbox;

	/** Holds the time at which the connection was opened. */
	FDateTime OpenedTime;

	/** Holds the collection of outbound payloads. */
	TSpscQueue<TSharedPtr<FArrayReader, ESPMode::ThreadSafe>> Outbox;

	/** Holds the IP endpoint of the remote client. */
	FIPv4Endpoint RemoteEndpoint;

	/** Holds the connection socket. */
	FSocket* Socket;

	/** Holds the thread object. */
	FRunnableThread* Thread;

	/** Holds the total number of bytes received from the connection. */
	uint64 TotalBytesReceived;

	/** Holds the total number of bytes sent to the connection. */
	uint64 TotalBytesSent;

	/** thread should continue running. */
	bool bRun;
};
