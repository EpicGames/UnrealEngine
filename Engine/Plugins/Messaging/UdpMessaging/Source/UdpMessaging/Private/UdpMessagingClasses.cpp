// Copyright Epic Games, Inc. All Rights Reserved.

#include "Shared/UdpMessagingSettings.h"


UUdpMessagingSettings::UUdpMessagingSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, EnableTransport(true)
	, MulticastTimeToLive(1)
	, EnableTunnel(false)
{ }
