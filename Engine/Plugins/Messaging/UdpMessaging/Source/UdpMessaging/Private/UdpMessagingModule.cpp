// Copyright Epic Games, Inc. All Rights Reserved.

#include "Transport/UdpMessageProcessor.h"

#include "MessageBridgeBuilder.h"
#include "Misc/CoreDelegates.h"
#include "Misc/App.h"

#if WITH_EDITOR
	#include "ISettingsModule.h"
	#include "ISettingsSection.h"
	#include "PropertyEditorModule.h"
	#include "Customization/UdpSettingsDetailsCustomization.h"
#endif

#include "Features/IModularFeatures.h"
#include "IUdpMessageTunnelConnection.h"
#include "Shared/UdpMessagingSettings.h"
#include "Transport/UdpMessageTransport.h"
#include "Tunnel/UdpMessageTunnel.h"


DEFINE_LOG_CATEGORY(LogUdpMessaging);

#define LOCTEXT_NAMESPACE "FUdpMessagingModule"


/**
 * Implements the UdpMessagingModule module and the network messaging extension modular feature.
 */
class FUdpMessagingModule
	: public FSelfRegisteringExec
	, public INetworkMessagingExtension
	, public IModuleInterface
{
protected:

	//~ FSelfRegisteringExec interface

	virtual bool Exec_Runtime(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override
	{
		if (!FParse::Command(&Cmd, TEXT("UDPMESSAGING")))
		{
			return false;
		}

		if (FParse::Command(&Cmd, TEXT("STATUS")))
		{
			const UUdpMessagingSettings* Settings = GetDefault<UUdpMessagingSettings>();

			// general information
			Ar.Logf(TEXT("Protocol Version: %i"), UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION);

			// bridge status
			if (MessageBridge.IsValid())
			{
				if (MessageBridge->IsEnabled())
				{
					Ar.Log(TEXT("Message Bridge: Initialized and enabled"));
				}
				else
				{
					Ar.Log(TEXT("Message Bridge: Initialized, but disabled"));
				}
			}
			else
			{
				Ar.Log(TEXT("Message Bridge: Not initialized."));
			}

			// bridge settings
			if (Settings->UnicastEndpoint.IsEmpty())
			{
				Ar.Logf(TEXT("    Unicast Endpoint: %s (default)"), *FIPv4Endpoint::Any.ToString());
			}
			else
			{
				Ar.Logf(TEXT("    Unicast Endpoint: %s"), *Settings->UnicastEndpoint);
			}

			if (Settings->MulticastEndpoint.IsEmpty())
			{
				Ar.Logf(TEXT("    Multicast Endpoint: %s (default)"), *UDP_MESSAGING_DEFAULT_MULTICAST_ENDPOINT.ToString());
			}
			else
			{
				Ar.Logf(TEXT("    Multicast Endpoint: %s"), *Settings->MulticastEndpoint);
			}

			Ar.Logf(TEXT("    Multicast TTL: %i"), Settings->MulticastTimeToLive);

			if (Settings->StaticEndpoints.Num() > 0)
			{
				Ar.Log(TEXT("    Static Endpoints:"));

				for (const auto& StaticEndpoint : Settings->StaticEndpoints)
				{
					Ar.Logf(TEXT("        %s"), *StaticEndpoint);
				}
			}
			else
			{
				Ar.Log(TEXT("    Static Endpoints: None"));
			}

#if PLATFORM_DESKTOP
			// tunnel status
			if (MessageTunnel.IsValid())
			{
				if (MessageTunnel->IsServerRunning())
				{
					Ar.Log(TEXT("Message Tunnel: Initialized and started"));
				}
				else
				{
					Ar.Log(TEXT("Message Tunnel: Initialized, but stopped"));
				}
			}
			else
			{
				Ar.Log(TEXT("Message Tunnel: Not initialized."));
			}

			// tunnel settings
			if (Settings->TunnelUnicastEndpoint.IsEmpty())
			{
				Ar.Logf(TEXT("    Unicast Endpoint: %s (default)"), *FIPv4Endpoint::Any.ToString());
			}
			else
			{
				Ar.Logf(TEXT("    Unicast Endpoint: %s"), *Settings->TunnelUnicastEndpoint);
			}

			if (Settings->TunnelMulticastEndpoint.IsEmpty())
			{
				Ar.Logf(TEXT("    Multicast Endpoint: %s (default)"), *UDP_MESSAGING_DEFAULT_MULTICAST_ENDPOINT.ToString());
			}
			else
			{
				Ar.Logf(TEXT("    Multicast Endpoint: %s"), *Settings->TunnelMulticastEndpoint);
			}

			if (Settings->RemoteTunnelEndpoints.Num() > 0)
			{
				Ar.Log(TEXT("    Remote Endpoints:"));

				for (const auto& RemoteEndpoint : Settings->RemoteTunnelEndpoints)
				{
					Ar.Logf(TEXT("        %s"), *RemoteEndpoint);
				}
			}
			else
			{
				Ar.Log(TEXT("    Remote Endpoints: None"));
			}

			// tunnel performance
			if (MessageTunnel.IsValid())
			{
				Ar.Logf(TEXT("    Total Bytes In: %i"), MessageTunnel->GetTotalInboundBytes());
				Ar.Logf(TEXT("    Total Bytes Out: %i"), MessageTunnel->GetTotalOutboundBytes());

				TArray<TSharedPtr<IUdpMessageTunnelConnection>> Connections;

				if (MessageTunnel->GetConnections(Connections) > 0)
				{
					Ar.Log(TEXT("  Active Connections:"));

					const FCoreTexts& CoreTexts = FCoreTexts::Get();
					for (const auto& Connection : Connections)
					{
						Ar.Logf(TEXT("  > %s, Open: %s, Uptime: %s, Bytes Received: %i, Bytes Sent: %i"),
							*Connection->GetName().ToString(),
							Connection->IsOpen() ? *CoreTexts.Yes.ToString() : *CoreTexts.No.ToString(),
							*Connection->GetUptime().ToString(),
							Connection->GetTotalBytesReceived(),
							Connection->GetTotalBytesSent()
						);
					}
				}
				else
				{
					Ar.Log(TEXT("  Active Connections: None"));
				}
			}
#endif
		}
		else if (FParse::Command(&Cmd, TEXT("RESTART")))
		{
			RestartServices();
		}
		else if (FParse::Command(&Cmd, TEXT("SHUTDOWN")))
		{
			ShutdownBridge();
#if PLATFORM_DESKTOP
			ShutdownTunnel();
#endif
		}
		else
		{
			// show usage
			Ar.Log(TEXT("Usage: UDPMESSAGING <Command>"));
			Ar.Log(TEXT(""));
			Ar.Log(TEXT("Command"));
			Ar.Log(TEXT("    RESTART = Restarts the message bridge and message tunnel, if enabled"));
			Ar.Log(TEXT("    SHUTDOWN = Shut down the message bridge and message tunnel, if running"));
			Ar.Log(TEXT("    STATUS = Displays the status of the UDP message transport"));
		}

		return true;
	}

public:

	// IModuleInterface interface
	virtual void StartupModule() override
	{
		if (!IsSupportEnabled())
		{
			return;
		}

		// load dependencies
		if (FModuleManager::Get().LoadModule(TEXT("Networking")) == nullptr)
		{
			UE_LOGF(LogUdpMessaging, Error, "The required module 'Networking' failed to load. Plug-in 'UDP Messaging' cannot be used.");

			return;
		}

		// Hook to the PreExit callback, needed to execute UObject related shutdowns
		FCoreDelegates::OnPreExit.AddRaw(this, &FUdpMessagingModule::HandleAppPreExit);

#if WITH_EDITOR
		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
		if (SettingsModule != nullptr)
		{
			ISettingsSectionPtr SettingsSection = SettingsModule->RegisterSettings("Project", "Plugins", "UdpMessaging",
				LOCTEXT("UdpMessagingSettingsName", "UDP Messaging"),
				LOCTEXT("UdpMessagingSettingsDescription", "Configure the UDP Messaging plug-in."),
				GetMutableDefault<UUdpMessagingSettings>()
			);

			if (SettingsSection.IsValid())
			{
				SettingsSection->OnModified().BindRaw(this, &FUdpMessagingModule::HandleSettingsSaved);
			}
		}

		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomClassLayout(UUdpMessagingSettings::StaticClass()->GetFName(), FOnGetDetailCustomizationInstance::CreateStatic(&FUdpSettingsDetailsCustomization::MakeInstance));
#endif // WITH_EDITOR

		// parse additional command line args
		ParseCommandLine(GetMutableDefault<UUdpMessagingSettings>(), FCommandLine::Get());

		// register application events
		const UUdpMessagingSettings& Settings = *GetDefault<UUdpMessagingSettings>();
		if (Settings.bStopServiceWhenAppDeactivates)
		{
			FCoreDelegates::ApplicationHasReactivatedDelegate.AddRaw(this, &FUdpMessagingModule::HandleApplicationHasReactivated);
			FCoreDelegates::ApplicationWillDeactivateDelegate.AddRaw(this, &FUdpMessagingModule::HandleApplicationWillDeactivate);
		}

		RestartServices();

		IModularFeatures::Get().RegisterModularFeature(INetworkMessagingExtension::ModularFeatureName, this);
	}

	virtual void ShutdownModule() override
	{
		// unregister application events
		FCoreDelegates::ApplicationHasReactivatedDelegate.RemoveAll(this);
		FCoreDelegates::ApplicationWillDeactivateDelegate.RemoveAll(this);

		// Unhook AppPreExit and call it
		FCoreDelegates::OnPreExit.RemoveAll(this);
		HandleAppPreExit();

#if WITH_EDITOR
		// unregister settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "UdpMessaging");
		}
#endif

		// shut down services
		ShutdownBridge();
#if PLATFORM_DESKTOP
		ShutdownTunnel();
#endif

		IModularFeatures::Get().UnregisterModularFeature(INetworkMessagingExtension::ModularFeatureName, this);
	}

	virtual bool SupportsDynamicReloading() override
	{
		return true;
	}

	// INetworkMessagingExtension interface
	virtual FName GetName() const
	{
		return UdpMessagingName;
	}

	virtual bool IsSupportEnabled() const
	{
#if  !IS_PROGRAM && UE_BUILD_SHIPPING && !(defined(ALLOW_UDP_MESSAGING_SHIPPING) && ALLOW_UDP_MESSAGING_SHIPPING)
		return false;
#else
		// disallow unsupported platforms
		if (!FPlatformMisc::SupportsMessaging())
		{
			return false;
		}

		// always allow in standalone Slate applications
		if (!FApp::IsGame() && !IsRunningCommandlet())
		{
			return true;
		}

		// otherwise allow if explicitly desired
		if (FParse::Param(FCommandLine::Get(), TEXT("Messaging")))
		{
			return true;
		}

		// check the project setting
		const UUdpMessagingSettings& Settings = *GetDefault<UUdpMessagingSettings>();
		return Settings.EnabledByDefault;
#endif
	}

	virtual void RestartServices()
	{
		const UUdpMessagingSettings& Settings = *GetDefault<UUdpMessagingSettings>();

		if (Settings.EnableTransport)
		{
			InitializeBridge();
		}
		else
		{
			ShutdownBridge();
		}

#if PLATFORM_DESKTOP
		if (Settings.EnableTunnel)
		{
			InitializeTunnel();
		}
		else
		{
			ShutdownTunnel();
		}
#endif
	}

	virtual void ShutdownServices()
	{
		ShutdownBridge();
#if PLATFORM_DESKTOP
		ShutdownTunnel();
#endif
		AdditionalStaticEndpoints.Empty();
	}

	virtual bool CanProvideNetworkStatistics() const override
	{
		return true;
	}

	virtual FMessageTransportStatistics GetLatestNetworkStatistics(FGuid NodeId) const override
	{
		if (auto Transport = WeakBridgeTransport.Pin())
		{
			return Transport->GetLatestStatistics(NodeId);
		}
		return {};
	}

	virtual FGuid GetNodeIdFromAddress(const FMessageAddress& MessageAddress) const override
	{
		return MessageBridge->LookupAddress(MessageAddress);
	}

	virtual FOnOutboundTransferDataUpdated& OnOutboundTransferUpdatedFromThread() override
	{
		return UE::Private::MessageProcessor::OnSegmenterUpdated();
	}

	virtual FOnInboundTransferDataUpdated& OnInboundTransferUpdatedFromThread() override
	{
		return UE::Private::MessageProcessor::OnReassemblerUpdated();
	}

	virtual TArray<FString> GetListeningAddresses() const override
	{
		if (auto Transport = WeakBridgeTransport.Pin())
		{
			TArray<FString> StringifiedEndpoints;
			TArray<FIPv4Endpoint> Endpoints = Transport->GetListeningAddresses();
			for (const FIPv4Endpoint& Endpoint : Endpoints)
			{
				StringifiedEndpoints.Add(Endpoint.ToString());
			}
			return StringifiedEndpoints;
		}
		return {};
	}

	virtual TArray<FString> GetKnownEndpoints() const override
	{
		if (auto Transport = WeakBridgeTransport.Pin())
		{
			TArray<FIPv4Endpoint> Endpoints = Transport->GetKnownEndpoints();
			TArray<FString> StringifiedEndpoints;
			for (const FIPv4Endpoint& Endpoint : Endpoints)
			{
				StringifiedEndpoints.Add(Endpoint.ToString());
			}
			return StringifiedEndpoints;
		}
		return {};
	}

	virtual void AddEndpoint(const FString& InEndpoint) override
	{
		if (auto Transport = WeakBridgeTransport.Pin())
		{
			FScopeLock StaticEndpointsLock(&StaticEndpointsCS);
			FIPv4Endpoint OutEndpoint;
			if (ParseEndpoint(InEndpoint, OutEndpoint) && !AdditionalStaticEndpoints.Contains(OutEndpoint))
			{
				AdditionalStaticEndpoints.Add(OutEndpoint);
				Transport->AddStaticEndpoint(OutEndpoint);
			}
		}
	}

	virtual void RemoveEndpoint(const FString& InEndpoint) override
	{
		if (auto Transport = WeakBridgeTransport.Pin())
		{
			FScopeLock StaticEndpointsLock(&StaticEndpointsCS);
			FIPv4Endpoint OutEndpoint;
			if (ParseEndpoint(InEndpoint,OutEndpoint) && AdditionalStaticEndpoints.Contains(OutEndpoint))
			{
				AdditionalStaticEndpoints.Remove(OutEndpoint);
				Transport->RemoveStaticEndpoint(OutEndpoint);
			}
		}
	}

protected:

	bool ParseEndpoint(const FString& InEndpointString, FIPv4Endpoint& OutEndpoint, bool bSupportHostname = true)
	{
		bool bParsedAddr = FIPv4Endpoint::Parse(InEndpointString, OutEndpoint);
		if (bSupportHostname && !bParsedAddr)
		{
			bParsedAddr = FIPv4Endpoint::FromHostAndPort(InEndpointString, OutEndpoint);
		}
		if (bParsedAddr)
		{
			// Detect 169.254.x.x addresses.
			if (OutEndpoint.Address.A == 169 &&	OutEndpoint.Address.B == 254)
			{
				UE_LOGF(LogUdpMessaging, Warning, "Detected IPv4 address in the form of 169.254.x.x. This is a link assigned address and may prevent you from reach external endpoints. ");
			}
		}
		return bParsedAddr;
	}

	TArray<FIPv4Endpoint> ParseStringArrayAddresses(const TArray<FString>& StringAddresses)
	{
		TArray<FIPv4Endpoint> Endpoints;
		for (const FString& EndpointAsString : StringAddresses)
		{
			FIPv4Endpoint Endpoint;

			if (ParseEndpoint(EndpointAsString, Endpoint))
			{
				Endpoints.Add(Endpoint);
			}
			else
			{
				UE_LOGF(LogUdpMessaging, Warning, "Invalid UDP Messaging Endpoint '%ls'", *EndpointAsString);
			}
		}
		return Endpoints;
	}

	/** Initializes the message bridge with the current settings. */
	void InitializeBridge()
	{
		ShutdownBridge();

		UUdpMessagingSettings* Settings = GetMutableDefault<UUdpMessagingSettings>();
		bool ResaveSettings = false;

		FIPv4Endpoint UnicastEndpoint;
		FIPv4Endpoint MulticastEndpoint;

		if (!ParseEndpoint(Settings->UnicastEndpoint, UnicastEndpoint))
		{
			if (!Settings->UnicastEndpoint.IsEmpty())
			{
				UE_LOGF(LogUdpMessaging, Warning, "Invalid setting for UnicastEndpoint '%ls' - binding to all local network adapters instead", *Settings->UnicastEndpoint);
			}

			UnicastEndpoint = FIPv4Endpoint::Any;
			Settings->UnicastEndpoint = UnicastEndpoint.ToText().ToString();
			ResaveSettings = true;
		}

		if (!ParseEndpoint(Settings->MulticastEndpoint, MulticastEndpoint, false))
		{
			if (!Settings->MulticastEndpoint.IsEmpty())
			{
				UE_LOGF(LogUdpMessaging, Warning, "Invalid setting for MulticastEndpoint '%ls' - using default endpoint '%ls' instead", *Settings->MulticastEndpoint, *UDP_MESSAGING_DEFAULT_MULTICAST_ENDPOINT.ToText().ToString());
			}

			MulticastEndpoint = UDP_MESSAGING_DEFAULT_MULTICAST_ENDPOINT;
			Settings->MulticastEndpoint = MulticastEndpoint.ToText().ToString();
			ResaveSettings = true;
		}

		// Initialize the service with the additional endpoints added through the modular interface
		TArray<FIPv4Endpoint> StaticEndpoints = AdditionalStaticEndpoints.Array();
		StaticEndpoints += ParseStringArrayAddresses(Settings->StaticEndpoints);

		if (Settings->MulticastTimeToLive == 0)
		{
			Settings->MulticastTimeToLive = 1;
			ResaveSettings = true;
		}

		if (ResaveSettings)
		{
			Settings->SaveConfig();
		}
		UE_LOGF(LogUdpMessaging, Log, "Initializing bridge on interface %ls to multicast group %ls.", *UnicastEndpoint.ToString(), *MulticastEndpoint.ToText().ToString());

		TSharedRef<FUdpMessageTransport, ESPMode::ThreadSafe> Transport = MakeShared<FUdpMessageTransport, ESPMode::ThreadSafe>(
			UnicastEndpoint, MulticastEndpoint, MoveTemp(StaticEndpoints), Settings->ExcludedEndpoints, Settings->MulticastTimeToLive);
		WeakBridgeTransport = Transport;
		MessageBridge = FMessageBridgeBuilder()
			.UsingTransport(Transport);
	}

#if PLATFORM_DESKTOP
	/** Initializes the message tunnel with the current settings. */
	void InitializeTunnel()
	{
		ShutdownTunnel();

		UUdpMessagingSettings* Settings = GetMutableDefault<UUdpMessagingSettings>();
		bool ResaveSettings = false;

		FIPv4Endpoint UnicastEndpoint;
		FIPv4Endpoint MulticastEndpoint;

		if (!ParseEndpoint(Settings->TunnelUnicastEndpoint, UnicastEndpoint))
		{
			if (!Settings->TunnelUnicastEndpoint.IsEmpty())
			{
				UE_LOGF(LogUdpMessaging, Warning, "Invalid setting for UnicastEndpoint '%ls' - binding to all local network adapters instead", *Settings->UnicastEndpoint);
			}

			UnicastEndpoint = FIPv4Endpoint::Any;
			Settings->TunnelUnicastEndpoint = UnicastEndpoint.ToString();
			ResaveSettings = true;
		}

		if (!ParseEndpoint(Settings->TunnelMulticastEndpoint, MulticastEndpoint, false))
		{
			if (!Settings->TunnelMulticastEndpoint.IsEmpty())
			{
				UE_LOGF(LogUdpMessaging, Warning, "Invalid setting for MulticastEndpoint '%ls' - using default endpoint '%ls' instead", *Settings->MulticastEndpoint, *UDP_MESSAGING_DEFAULT_MULTICAST_ENDPOINT.ToText().ToString());
			}

			MulticastEndpoint = UDP_MESSAGING_DEFAULT_MULTICAST_ENDPOINT;
			Settings->TunnelMulticastEndpoint = MulticastEndpoint.ToString();
			ResaveSettings = true;
		}

		if (ResaveSettings)
		{
			Settings->SaveConfig();
		}

		UE_LOGF(LogUdpMessaging, Log, "Initializing tunnel on interface %ls to multicast group %ls.", *UnicastEndpoint.ToString(), *MulticastEndpoint.ToText().ToString());

		MessageTunnel = MakeShareable(new FUdpMessageTunnel(UnicastEndpoint, MulticastEndpoint));

		// initiate connections
		for (int32 EndpointIndex = 0; EndpointIndex < Settings->RemoteTunnelEndpoints.Num(); ++EndpointIndex)
		{
			FIPv4Endpoint RemoteEndpoint;

			if (ParseEndpoint(Settings->RemoteTunnelEndpoints[EndpointIndex], RemoteEndpoint))
			{
				MessageTunnel->Connect(RemoteEndpoint);
			}
			else
			{
				UE_LOGF(LogUdpMessaging, Warning, "Invalid UDP RemoteTunnelEndpoint '%ls' - skipping", *Settings->RemoteTunnelEndpoints[EndpointIndex]);
			}
		}
	}
#endif

	/**
	 * Parse command line arguments to override UdpMessagingSettings
	 */
	void ParseCommandLine(UUdpMessagingSettings* Settings, const TCHAR* CommandLine)
	{
		if (Settings && CommandLine)
		{
			// Parse value overrides (if present)
			FParse::Bool(CommandLine, TEXT("-UDPMESSAGING_TRANSPORT_ENABLE="), Settings->EnableTransport);
			FParse::Bool(CommandLine, TEXT("-UDPMESSAGING_SHARE_KNOWN_NODES="), Settings->bShareKnownNodesWithActiveConnections);
			FParse::Value(CommandLine, TEXT("-UDPMESSAGING_TRANSPORT_UNICAST="), Settings->UnicastEndpoint);
			FParse::Value(CommandLine, TEXT("-UDPMESSAGING_TRANSPORT_MULTICAST="), Settings->MulticastEndpoint);

			FString StaticEndpoints;
			FParse::Value(CommandLine, TEXT("-UDPMESSAGING_TRANSPORT_STATIC="), StaticEndpoints, false);
			TArray<FString> CommandLineStaticEndpoints;
			StaticEndpoints.ParseIntoArray(CommandLineStaticEndpoints, TEXT(","));
			for (const FString& CmdStaticEndpoint : CommandLineStaticEndpoints)
			{
				Settings->StaticEndpoints.AddUnique(CmdStaticEndpoint);
			}

			if (Settings->bShareKnownNodesWithActiveConnections)
			{
				UE_LOGF(LogUdpMessaging, Display, "Sharing known nodes with endpoints is enabled.");
			}
		}
	}

	/** Shuts down the message bridge. */
	void ShutdownBridge()
	{
		WeakBridgeTransport.Reset();
		if (MessageBridge.IsValid())
		{
			MessageBridge->Disable();
			FPlatformProcess::Sleep(0.1f);
			MessageBridge.Reset();
		}
	}

#if PLATFORM_DESKTOP
	/** Shuts down the message tunnel. */
	void ShutdownTunnel()
	{
		if (MessageTunnel.IsValid())
		{
			MessageTunnel->StopServer();
			MessageTunnel.Reset();
		}
	}
#endif

private:

	/** Callback for when an has been reactivated (i.e. return from sleep on iOS). */
	void HandleApplicationHasReactivated()
	{
		RestartServices();
	}

	/** Callback for when the application will be deactivated (i.e. sleep on iOS).*/
	void HandleApplicationWillDeactivate()
	{
		ShutdownServices();
	}

	/** Callback for when the settings were saved. */
	bool HandleSettingsSaved()
	{
		RestartServices();

		return true;
	}

	void HandleAppPreExit()
	{
		if (TSharedPtr<FUdpMessageTransport, ESPMode::ThreadSafe> UdpTransport = WeakBridgeTransport.Pin())
		{
			UdpTransport->OnAppPreExit();
		}
	}

private:
	/** Name of the network messaging extension. */
	static FName UdpMessagingName;

	/** Holds the message bridge if present. */
	TSharedPtr<IMessageBridge, ESPMode::ThreadSafe> MessageBridge;

	/** Holds the bridge transport if present.  */
	TWeakPtr<FUdpMessageTransport, ESPMode::ThreadSafe> WeakBridgeTransport;

	/** Critical section protecting access to the transport static endpoints and additional static endpoints. */
	FCriticalSection StaticEndpointsCS;

	/** Holds additional static endpoints added through the modular feature interface. */
	TSet<FIPv4Endpoint> AdditionalStaticEndpoints;

#if PLATFORM_DESKTOP
	/** Holds the message tunnel if present. */
	TSharedPtr<IUdpMessageTunnel> MessageTunnel;
#endif
};

FName FUdpMessagingModule::UdpMessagingName("UdpMessaging");


void EmptyLinkFunctionForStaticInitializationUdpMessagingTests()
{
	// Force references to the object files containing the functions below, to prevent them being
	// excluded by the linker when the plug-in is compiled into a static library for monolithic builds.
	extern void EmptyLinkFunctionForStaticInitializationUdpMessageSegmenterTest();
	EmptyLinkFunctionForStaticInitializationUdpMessageSegmenterTest();
	extern void EmptyLinkFunctionForStaticInitializationUdpMessageTransportTest();
	EmptyLinkFunctionForStaticInitializationUdpMessageTransportTest();
	extern void EmptyLinkFunctionForStaticInitializationUdpSerializeMessageTaskTest();
	EmptyLinkFunctionForStaticInitializationUdpSerializeMessageTaskTest();
}


IMPLEMENT_MODULE(FUdpMessagingModule, UdpMessaging);


#undef LOCTEXT_NAMESPACE
