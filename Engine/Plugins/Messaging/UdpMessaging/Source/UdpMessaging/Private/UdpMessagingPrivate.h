// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"


/** Declares a log category for this module. */
DECLARE_LOG_CATEGORY_EXTERN(LogUdpMessaging, Log, All);


/** Defines the default IP endpoint for multicast traffic. */
#define UDP_MESSAGING_DEFAULT_MULTICAST_ENDPOINT FIPv4Endpoint(FIPv4Address(230, 0, 0, 1), 6666)

/** Defines the maximum number of annotations a message can have. */
#define UDP_MESSAGING_MAX_ANNOTATIONS 128

/** Defines the maximum number of recipients a message can have. */
#define UDP_MESSAGING_MAX_RECIPIENTS 1024

/** Defines the desired size of socket receive buffers (in bytes). */
#define UDP_MESSAGING_RECEIVE_BUFFER_SIZE 2 * 1024 * 1024

/**
 * Defines the protocol version of the UDP message transport.
 * @note When changing the version, ensure to update the serialization/deserialization code in UdpSerializeMessageTask.cpp/UdpDeserializedMessage.cpp and the supported versions in FUdpMessageProcessor::Init().
 */
#define UDP_MESSAGING_TRANSPORT_PROTOCOL_VERSION 18


namespace UE::UdpMessaging
{
	/********************************
	 *
	 *   UDP Messaging Terminology
	 *
	 ********************************
	 *
	 *   Protocol Data Unit (PDU):
	 *       Atomic unit of data, including any headers, payload, or trailers used by the protocol.
	 *
	 *   Service Data Unit (SDU):
	 *       The usable payload component of the PDU. Populated by the next higher layer.
	 *
	 *   Maximum Transmission Unit (MTU):
	 *       The largest allowable size for a PDU at a given layer.
	 *
	 *   Fragmentation:
	 *       The IP layer transparently dividing and reassembling your oversized payload into multiple PDUs.
	 *
	 *   Segmentation:
	 *       Trying to respect the lower layer's MTU and avoid fragmentation by proactively dividing payload.
	 *       Can be (hardware) accelerated, such as by UDP segmentation offload (USO).
	 *
	 *   Frame:
	 *       Ethernet / OSI layer 2 PDU. Typical MTU of 1500 bytes. "Jumbo frame" support allows MTU of ~9000 bytes.
	 *
	 *   Packet:
	 *       IP / OSI layer 3 PDU. Usable MTU depends on routing path; ~1200 bytes is generally internet safe.
	 *
	 *   Datagram:
	 *       UDP / OSI layer 4 PDU. Constitutes the SDU of the IP packet.
	 * 
	 *   Segment:
	 *       UDP Messaging PDU (You are here). Encapsulated within (SDU of) a UDP datagram.
	 * 
	 */

	// Technically minimum, but almost always holds in practice.
	constexpr uint16 Ipv4HeaderBytes = 20;
	constexpr uint16 UdpHeaderBytes = 8;
	constexpr uint16 PacketHeaderBytes = Ipv4HeaderBytes + UdpHeaderBytes;
}
