// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

class IUdpMessageTunnelConnection;

struct FIPv4Endpoint;


/**
 * Interface for message tunnels.
 */
class IUdpMessageTunnel
{
public:

	/**
	 * Establishes a tunnel with a remote server.
	 *
	 * @param RemoteEndpoint The endpoint of the server to connect to.
	 * @return true if the connection has been established, false otherwise.
	 * @see GetConnections
	 */
	virtual bool Connect(const FIPv4Endpoint& RemoteEndpoint) = 0;

	/** 
	 * Starts the tunnel server.
	 *
	 * @param LocalEndpoint The IP endpoint to listen for incoming connections on.
	 * @see IsServerRunning, StopServer
	 */
	virtual void StartServer(const FIPv4Endpoint& LocalEndpoint) = 0;

	/**
	 * Stops the tunnel server.
	 *
	 * @see IsServerRunning, StartServer
	 */
	virtual void StopServer() = 0;

public:

	/**
	 * Gets the list of all open tunnel connections.
	 *
	 * @param OutConnections Will hold the list of connections.
	 * @return The number of connections returned.
	 * @see Connect
	 */
	virtual int32 GetConnections(TArray<TSharedPtr<IUdpMessageTunnelConnection>>& OutConnections) = 0;

	/**
	 * Gets the total number of bytes that were received from tunnels.
	 *
	 * @return Number of bytes.
	 */
	virtual uint64 GetTotalInboundBytes() const = 0;

	/**
	 * Gets the total number of bytes that were sent out through tunnels.
	 *
	 * @return Number of bytes.
	 */
	virtual uint64 GetTotalOutboundBytes() const = 0;

	/**
	 * Checks whether the tunnel server is running.
	 *
	 * @return true if the tunnel server is running, false otherwise.
	 * @see StartServer, StopServer
	 */
	virtual bool IsServerRunning() const = 0;

public:

	/**
	 * Gets a delegate that is executed when the list of incoming connections changed.
	 *
	 * @return The delegate.
	 */
	virtual FSimpleDelegate& OnConnectionsChanged() = 0;

public:

	/** Virtual destructor. */
	virtual ~IUdpMessageTunnel() = default;
};
