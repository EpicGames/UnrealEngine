// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class FText;
struct FTimespan;


/**
 * Interface for message tunnel connections.
 */
class IUdpMessageTunnelConnection
{
public:

	/** Closes this connection. */
	virtual void Close() = 0;

	/**
	 * Gets the total number of bytes received from this connection.
	 *
	 * @return Number of bytes.
	 */
	virtual uint64 GetTotalBytesReceived() const = 0;

	/**
	 * Gets the total number of bytes received from this connection.
	 *
	 * @return Number of bytes.
	 */
	virtual uint64 GetTotalBytesSent() const = 0;

	/**
	 * Gets the human readable name of the connection.
	 *
	 * @return Connection name.
	 */
	virtual FText GetName() const = 0;

	/**
	 * Gets the amount of time that the connection has been established.
	 *
	 * @return Time span.
	 */
	virtual FTimespan GetUptime() const = 0;

	/**
	 * Checks whether this connection is open.
	 *
	 * @return true if the connection is open, false otherwise.
	 */
	virtual bool IsOpen() const = 0;

public:

	/** Virtual destructor. */
	virtual ~IUdpMessageTunnelConnection() = default;
};
