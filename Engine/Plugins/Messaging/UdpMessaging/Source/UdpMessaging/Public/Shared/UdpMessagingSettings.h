// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


#include "UdpMessagingSettings.generated.h"

/** Defines the UDP message format available (how the message data is encoded). */
UENUM()
enum class EUdpMessageFormat : uint8
{
	/** No format specified. Legacy - Not exposed to user.*/
	None = 0 UMETA(Hidden),

	/** JSON format specified. Legacy - Not exposed to user.*/
	Json UMETA(Hidden),

	/** Tagged property format specified. Legacy - Not exposed to user.*/
	TaggedProperty UMETA(Hidden),

	/**
	 * UDP messages are encoded in CBOR, using the platform endianness. This is the fastest and preferred option, but the CBOR data will not be readable by an external standard-compliant CBOR parser
	 * if generated from a little endian platform. If the data needs to be consumed outside the Unreal Engine, consider using CborStandardEndianness format instead.
	 */
	CborPlatformEndianness UMETA(DisplayName="CBOR (Platform Endianness)"),

	/**
	 * UDP messages are encoded in CBOR, using the CBOR standard-complinant endianness (big endian). It will perform slower on a little-endian platform, but the data will be readable by standard CBOR parsers.
	 * Useful if the UDP messages needs to be analyzed/consumed outside the Unreal Engine.
	 */
	CborStandardEndianness UMETA(DisplayName="CBOR (Standard Endianness)"),
};

UCLASS(config=Engine)
class UUdpMessagingSettings
	: public UObject
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Whether UDP messaging is enabled by default. If false -messaging will need to be added
	 * to the commandline when running non-editor builds.
	 *
	 * (Note - in Shipping builds ALLOW_UDP_MESSAGING_SHIPPING=1 must also be defined in TargetRules
	 * for messaging to be available with or without this setting)
	 */
	UPROPERTY(config, EditAnywhere, Category = Availability)
	bool EnabledByDefault = false;

	/**
	 * Whether the UDP transport channel is enabled.
	 * Can be specified on the command line with `-UDPMESSAGING_TRANSPORT_ENABLE=`
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport)
	bool EnableTransport;

	/** Whether the UDP transport channel should try to auto repair when in error. */
	UPROPERTY(config, EditAnywhere, Category=Transport, AdvancedDisplay)
	bool bAutoRepair = true;

	/** 
	 * Maximum sustained transmission rate in Gbit / s. 
	 *
	 * The default value is 1 Gbit/s.
	 *
	 * This is to control transmission of larger packages over the network. Without a limit, it is
	 * possible for packet data loss to occur because more data may be sent then may be supported
	 * by your network card.
	 *
	 * Adjust this value higher or lower depending on your network capacity.
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport, meta = (ClampMin="0.01", ClampMax="100.0"), AdvancedDisplay)
	float MaxSendRate = 1.0f;

	static constexpr float MaxSendRate_Min = 0.01f;
	static constexpr float MaxSendRate_Max = 100.0f;

	/** Accessor that ensures MaxSendRate is clamped to the expected bounds. */
	float GetMaxSendRate() const
	{
		return FMath::Clamp<float>(MaxSendRate, MaxSendRate_Min, MaxSendRate_Max);
	}

	/** The number of consecutive attempt the auto repair routine will try to repair. */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	uint32 AutoRepairAttemptLimit = 10;

	UE_DEPRECATED(5.8, "UDP Messaging work queues are no longer fixed size.")
	UPROPERTY(meta=(DeprecatedProperty))
	uint16 WorkQueueSize = 1024;

	/**
	 * The maximum packet size to use for message segments (including presumed 20 byte IPv4 + 8 byte UDP headers).
	 * This should be less than or equal to your path maximum transmission unit (PMTU) to avoid fragmentation.
	 * Previous versions of UDP Messaging were hard-coded to 1130, which is a conservative, internet-safe value.
	 *
	 * If your traffic is restricted to the local network, support for an MTU of 1500 is common.
	 * This can be increased further by enabling jumbo frames on supported hardware.
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport, meta = (ClampMin="576", ClampMax="65535"), AdvancedDisplay)
	uint16 MaxPacketSize = 1130;

	static constexpr uint16 MaxPacketSize_Min = 576;
	static constexpr uint16 MaxPacketSize_Max = 65535;

	/** Accessor that ensures MaxPacketSize is clamped to the expected bounds. */
	uint16 GetMaxPacketSize() const
	{
		return FMath::Clamp<uint16>(MaxPacketSize, MaxPacketSize_Min, MaxPacketSize_Max);
	}

	/** Enable UDP segmentation offload. */
	UPROPERTY(config, EditAnywhere, Category = Transport, AdvancedDisplay)
	bool bSegmentationOffload = false;

	/**
	 * Reliable queue priority specified as a percentage with a range from 5-95%. Setting values above or below
	 * can completely starve unreliable messages.
	 */
	UPROPERTY(config, EditAnywhere, Category = Transport, meta = (ClampMin="5", ClampMax="95"), AdvancedDisplay)
	float ReliableQueuePriority = 75.0f;

	/** Whether to stop the transport service when the application deactivates, and restart it when the application is reactivated */
	UPROPERTY(config)
	bool bStopServiceWhenAppDeactivates = true;

	/**
	 * The IP endpoint to listen to and send packets from.
	 *
	 * The format is IP_ADDRESS:PORT_NUMBER.
	 * 0.0.0.0:0 will bind to the default network adapter on Windows,
	 * and all available network adapters on other operating systems.
	 * Specifying an interface IP here, will use that interface for multicasting and static endpoint *might* also reach this client through <unicast ip:multicast port>
	 * Specifying both the IP and Port will allow usage of static endpoint to reach this client
	 * Can be specified on the command line with `-UDPMESSAGING_TRANSPORT_UNICAST=`
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport)
	FString UnicastEndpoint;

	/**
	 * The IP endpoint to send multicast packets to.
	 *
	 * The format is IP_ADDRESS:PORT_NUMBER.
	 * The multicast IP address must be in the range 224.0.0.0 to 239.255.255.255.
	 * Can be specified on the command line with `-UDPMESSAGING_TRANSPORT_MULTICAST=`
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport)
	FString MulticastEndpoint;

	/** The format used to serialize the UDP message payload. */
	UPROPERTY(config, EditAnywhere, Category=Transport)
	EUdpMessageFormat MessageFormat = EUdpMessageFormat::CborPlatformEndianness;

	/** The time-to-live (TTL) for sent multicast packets. */
	UPROPERTY(config, EditAnywhere, Category=Transport, AdvancedDisplay)
	uint8 MulticastTimeToLive;

	/**
	 * The IP endpoints of static devices.
	 *
	 * Use this setting to reach devices on other subnets, such as mobile phones on a WiFi network.
	 * The format is IP_ADDRESS:PORT_NUMBER.
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport, AdvancedDisplay)
	TArray<FString> StaticEndpoints;

	/**
	 * List of IP addresses that are banned from communicating with this client. Wildcard IPv4 addresses can be specified by using '*' and '?'
	 * characters.
	 *
	 * For example, 192.168.12.* will deny all addresses on the .12 subnet for the class C network.
	 * The string 192.16?.12.10 will deny all addresses in the range 192.160.12.10-192.169.12.10.
	 *
	 * The format is IP_ADDRESS:PORT_NUMBER.  If port number is 0 then all ports are blocked.
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport, AdvancedDisplay)
	TArray<FString> ExcludedEndpoints;

	/**
	 * Whether to share our list of known endpoints with those endpoints.  This allows message processors to completely mesh their networks.
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport, AdvancedDisplay)
	bool bShareKnownNodesWithActiveConnections = false;

	/**
	 * Maximum number of deserialization tasks that can be performed in parallel. A value of 0 or 1 means that concurrent
	 * deserialization is disabled, which will make the message processor behave similar to releases pre-5.6.
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport, meta = (ClampMax="32"), AdvancedDisplay)
	uint32 MaxConcurrentDeserializationTasks = 8;

	/**
	 * The period of time to consider an active connection to be timed-out. A regular heart beat is expected from connected endpoints and this will
	 * expire that connection if no data is received after a period of time.  If zero is specified the endpoints are never timed out (this should
	 * only be used for debugging purposes).
	 */
	UPROPERTY(config, EditAnywhere, Category=Transport, meta = (ClampMin="0"), AdvancedDisplay)
	float ConnectionTimeoutPeriod = 5.0f;

public:

	/** Whether the UDP tunnel is enabled. */
	UPROPERTY(config, EditAnywhere, Category=Tunnel)
	bool EnableTunnel;

	/**
	 * The local IP endpoint to listen to and send packets from.
	 *
	 * The format is IP_ADDRESS:PORT_NUMBER.
	 */
	UPROPERTY(config, EditAnywhere, Category=Tunnel)
	FString TunnelUnicastEndpoint;

	/**
	 * The IP endpoint to send multicast packets to.
	 *
	 * The format is IP_ADDRESS:PORT_NUMBER.
	 * The multicast IP address must be in the range 224.0.0.0 to 239.255.255.255.
	 */
	UPROPERTY(config, EditAnywhere, Category=Tunnel)
	FString TunnelMulticastEndpoint;

	/**
	 * The IP endpoints of remote tunnel nodes.
	 *
	 * Use this setting to connect to remote tunnel services.
	 * The format is IP_ADDRESS:PORT_NUMBER.
	 */
	UPROPERTY(config, EditAnywhere, Category=Tunnel, AdvancedDisplay)
	TArray<FString> RemoteTunnelEndpoints;
};
