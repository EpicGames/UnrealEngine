// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/CoreMiscDefines.h"

UE_DEPRECATED_HEADER(5.8, "SCOPED_MESSAGING_TRACE and MessagingChannel are now exported by MessagingTrace.h in the Messaging module.")

#include "MessagingTrace.h"
