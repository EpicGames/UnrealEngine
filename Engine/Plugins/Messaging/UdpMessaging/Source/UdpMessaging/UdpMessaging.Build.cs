// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class UdpMessaging : ModuleRules
	{
		public UdpMessaging(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"Networking",
					"AtomicQueue"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"CoreUObject",
					"Json",
					"Cbor",
					"Messaging",
					"Projects",
					"Serialization",
					"TraceLog",
					"Sockets"
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"MessagingCommon",
				});

			if (Target.bCompileAgainstEditor)
			{
				DynamicallyLoadedModuleNames.AddRange(
					new string[] {
						"Settings",
					});
				
				PrivateIncludePathModuleNames.AddRange(
					new string[] {
						"Settings",
					});

				PrivateDependencyModuleNames.AddRange(
					new string[] {
						"ApplicationCore",
						"EditorStyle",
						"Engine",
						"InputCore",
						"PropertyEditor",
						"Slate",
						"SlateCore",
						"UnrealEd"
					});
			}
		}
	}
}
