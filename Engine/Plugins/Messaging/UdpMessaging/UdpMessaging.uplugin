{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "UDP Messaging",
	"Description" : "Adds a UDP based transport and tunneling layer to the messaging sub-system for sending and receiving messages between networked computers and devices.",
	"Category" : "Messaging",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"SupportedPrograms": [
		"UnrealInsights",
		"UnrealFrontend",
		"UnrealMultiUserServer",
		"UnrealMultiUserSlateServer",
		"UnrealRecoverySvc",
		"UnrealLightmass",
		"CrashReportClientEditor",
		"LiveLinkHub",
		"TimeSynchronizationWriter",
		"LiveLinkProviderSample",
		"SpatialMetricsProfiler",
		"ChaosVisualDebugger"
	],
	"Modules" :
	[
		{
			"Name" : "UdpMessaging",
			"Type" : "RuntimeAndProgram",
			"LoadingPhase" : "PreDefault",
			"PlatformDenyList" :
			[
			],
			"ProgramAllowList": [
				"UnrealInsights",
				"UnrealFrontend",
				"UnrealMultiUserServer",
				"UnrealMultiUserSlateServer",
				"UnrealRecoverySvc",
				"UnrealLightmass",
				"CrashReportClientEditor",
				"LiveLinkHub",
				"TimeSynchronizationWriter",
				"LiveLinkProviderSample",
				"SpatialMetricsProfiler",
				"ChaosVisualDebugger"
			]
		}
	]
}
